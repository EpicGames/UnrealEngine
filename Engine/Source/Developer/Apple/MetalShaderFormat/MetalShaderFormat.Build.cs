// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class MetalShaderFormat : ModuleRules
{
	public MetalShaderFormat(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;
		
		PrivateIncludePathModuleNames.Add("TargetPlatform");
		PublicIncludePathModuleNames.Add("MetalRHI");

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"RenderCore",
				"ShaderCompilerCommon",
				"HlslParser",
				"ShaderPreprocessor",
				"FileUtilities",
				"RHI",
				"Shaders"
			}
			);

		if (Target.Platform == UnrealTargetPlatform.Mac || Target.Platform == UnrealTargetPlatform.Win64)
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target, "SPIRVReflect");
            AddEngineThirdPartyPrivateStaticDependencies(Target, "MetalShaderConverter");

			if (Target.Platform == UnrealTargetPlatform.Mac)
			{
				string SCBinariesDir = Path.Combine(Target.UEThirdPartyBinariesDirectory, "ShaderConductor", "Mac");
				PublicAdditionalLibraries.Add(SCBinariesDir + "/libdxcompiler.dylib");
				RuntimeDependencies.Add(SCBinariesDir + "/libdxcompiler.dylib");
			}
			else if (Target.Platform == UnrealTargetPlatform.Win64)
			{
				string ArchDir = (Target.Architecture == UnrealArch.Arm64) ? "WinArm64" : "Win64";
				RuntimeDependencies.Add(Path.Combine(Target.UEThirdPartyBinariesDirectory, "ShaderConductor", ArchDir, "dxcompiler.dll"));
			}

			PublicSystemIncludePaths.Add(Path.Combine(Target.UEThirdPartySourceDirectory, "ShaderConductor", "ShaderConductor", "External", "DirectXShaderCompiler", "include"));
		}
	}
}
