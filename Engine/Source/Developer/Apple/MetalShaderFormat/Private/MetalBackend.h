// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include <list>
#include <map>

enum EMetalGPUSemantics
{
	EMetalGPUSemanticsMobile, // Mobile shaders for TBDR GPUs
	EMetalGPUSemanticsTBDRDesktop, // Desktop shaders for TBDR GPUs
	EMetalGPUSemanticsImmediateDesktop // Desktop shaders for Immediate GPUs
};

// Metal supports 16 across all HW
static const int32 MaxMetalSamplers = 16;

struct FShaderCompilerEnvironment;

struct SDMARange
{
	unsigned SourceCB;
	unsigned SourceOffset;
	unsigned Size;
	unsigned DestCBIndex;
	unsigned DestCBPrecision;
	unsigned DestOffset;
	
	bool operator <(SDMARange const & Other) const
	{
		if (SourceCB == Other.SourceCB)
		{
			return SourceOffset < Other.SourceOffset;
		}
		
		return SourceCB < Other.SourceCB;
	}
};
typedef std::list<SDMARange> TDMARangeList;
typedef std::map<unsigned, TDMARangeList> TCBDMARangeMap;


static void InsertRange( TCBDMARangeMap& CBAllRanges, unsigned SourceCB, unsigned SourceOffset, unsigned Size, unsigned DestCBIndex, unsigned DestCBPrecision, unsigned DestOffset )
{
	check(SourceCB < (1 << 12));
	check(DestCBIndex < (1 << 12));
	check(DestCBPrecision < (1 << 8));
	unsigned SourceDestCBKey = (SourceCB << 20) | (DestCBIndex << 8) | DestCBPrecision;
	SDMARange Range = { SourceCB, SourceOffset, Size, DestCBIndex, DestCBPrecision, DestOffset };
	
	TDMARangeList& CBRanges = CBAllRanges[SourceDestCBKey];
	//printf("* InsertRange: %08x\t%u:%u - %u:%c:%u:%u\n", SourceDestCBKey, SourceCB, SourceOffset, DestCBIndex, DestCBPrecision, DestOffset, Size);
	if (CBRanges.empty())
	{
		CBRanges.push_back(Range);
	}
	else
	{
		TDMARangeList::iterator Prev = CBRanges.end();
		bool bAdded = false;
		for (auto Iter = CBRanges.begin(); Iter != CBRanges.end(); ++Iter)
		{
			if (SourceOffset + Size <= Iter->SourceOffset)
			{
				if (Prev == CBRanges.end())
				{
					CBRanges.push_front(Range);
				}
				else
				{
					CBRanges.insert(Iter, Range);
				}
				
				bAdded = true;
				break;
			}
			
			Prev = Iter;
		}
		
		if (!bAdded)
		{
			CBRanges.push_back(Range);
		}
		
		if (CBRanges.size() > 1)
		{
			// Try to merge ranges
			bool bDirty = false;
			do
			{
				bDirty = false;
				TDMARangeList NewCBRanges;
				for (auto Iter = CBRanges.begin(); Iter != CBRanges.end(); ++Iter)
				{
					if (Iter == CBRanges.begin())
					{
						Prev = CBRanges.begin();
					}
					else
					{
						if (Prev->SourceOffset + Prev->Size == Iter->SourceOffset && Prev->DestOffset + Prev->Size == Iter->DestOffset)
						{
							SDMARange Merged = *Prev;
							Merged.Size = Prev->Size + Iter->Size;
							NewCBRanges.pop_back();
							NewCBRanges.push_back(Merged);
							++Iter;
							NewCBRanges.insert(NewCBRanges.end(), Iter, CBRanges.end());
							bDirty = true;
							break;
						}
					}
					
					NewCBRanges.push_back(*Iter);
					Prev = Iter;
				}
				
				CBRanges.swap(NewCBRanges);
			}
			while (bDirty);
		}
	}
}
