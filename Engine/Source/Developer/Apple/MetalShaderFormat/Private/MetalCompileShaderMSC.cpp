// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetalCompileShaderMSC.h"
#include "MetalShaderCompiler.h"

#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/Compression.h"
#include "Misc/OutputDeviceRedirector.h"
#include "MetalBackend.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "RHIDefinitions.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "ShaderCompilerDefinitions.h"
#include "SpirvReflectCommon.h"
#include "ShaderParameterParser.h"
#include "Containers/AnsiString.h"
#include "ShaderCompilerCommonInternal.h"
#include "RayTracingBuiltInResources.h"
#include "ShaderSDCE.h"

#include <regex>

#if PLATFORM_MAC || PLATFORM_WINDOWS
THIRD_PARTY_INCLUDES_START
#include "metal_irconverter.h"
THIRD_PARTY_INCLUDES_END

extern void BuildMetalShaderOutput(
	FShaderCompilerOutput& ShaderOutput,
	const FShaderCompilerInput& ShaderInput,
	const FShaderParameterParser& ShaderParameterParser,
	const ANSICHAR* InShaderSource,
	uint32 SourceLen,
	uint32 SourceCRCLen,
	uint32 SourceCRC,
	uint32 Version,
	TCHAR const* Standard,
	TCHAR const* MinOSVersion,
	TArray<FShaderCompilerError>& OutErrors,
	uint32 TypedBuffers,
	uint32 InvariantBuffers,
	uint32 TypedUAVs,
	uint32 ConstantBuffers,
	bool bAllowFastIntriniscs,
	uint32 NumCBVs,
	uint32 OutputSizeVS,
	uint32 MaxInputPrimitivesPerMeshThreadgroupGS,
	const bool bUsesDiscard,
	char const* ShaderReflectionJSON,
	FMetalShaderBytecode const& CompiledShaderBytecode,
	FMetalBindlessShaderInfo* ShaderInfo
);

#include "ShaderConductorContext.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#endif
	#include "d3d12shader.h"
	#include "dxc/dxcapi.h"
#if PLATFORM_WINDOWS
	#include <dxc/Support/dxcapi.use.h>
#include "Windows/HideWindowsPlatformTypes.h"
#endif

inline IRShaderVisibility ShaderFrequencyToVisibility(const EShaderFrequency UEStage)
{
	switch (UEStage)
	{
	case SF_Vertex        : return IRShaderVisibilityVertex;
	case SF_Mesh          : return IRShaderVisibilityMesh;
	case SF_Amplification : return IRShaderVisibilityAmplification;
	case SF_Pixel         : return IRShaderVisibilityPixel;
	case SF_Geometry      : return IRShaderVisibilityGeometry;
	case SF_Compute       : return IRShaderVisibilityAll;
	case SF_RayGen        : return IRShaderVisibilityAll;
	case SF_RayMiss       : return IRShaderVisibilityAll;
	case SF_RayHitGroup   : return IRShaderVisibilityAll;
	case SF_RayCallable   : return IRShaderVisibilityAll;
	default               : checkNoEntry();
	}
	return IRShaderVisibilityAll;
}

inline IRResourceType QuantizeD3DResourceType(const D3D_SHADER_INPUT_TYPE Type)
{
	switch (Type)
	{
	case D3D_SIT_CBUFFER:                         return IRResourceTypeCBV;
	case D3D_SIT_TBUFFER:                         return IRResourceTypeCBV;
	case D3D_SIT_TEXTURE:                         return IRResourceTypeSRV;
	case D3D_SIT_SAMPLER:                         return IRResourceTypeSampler;
	case D3D_SIT_UAV_RWTYPED:                     return IRResourceTypeUAV;
	case D3D_SIT_STRUCTURED:                      return IRResourceTypeSRV;
	case D3D_SIT_UAV_RWSTRUCTURED:                return IRResourceTypeUAV;
	case D3D_SIT_BYTEADDRESS:                     return IRResourceTypeSRV;
	case D3D_SIT_UAV_RWBYTEADDRESS:               return IRResourceTypeUAV;
	case D3D_SIT_UAV_APPEND_STRUCTURED:           return IRResourceTypeUAV;
	case D3D_SIT_UAV_CONSUME_STRUCTURED:          return IRResourceTypeUAV;
	case D3D_SIT_UAV_RWSTRUCTURED_WITH_COUNTER:   return IRResourceTypeUAV;
	case D3D_SIT_RTACCELERATIONSTRUCTURE:         return IRResourceTypeSRV;
	case D3D_SIT_UAV_FEEDBACKTEXTURE:             return IRResourceTypeUAV;
	default:                                      checkNoEntry();
	}

	return IRResourceTypeInvalid;
}

inline bool IsD3DResourceTypeTyped(const D3D_SHADER_INPUT_TYPE Type)
{
	return Type == D3D_SIT_TBUFFER || Type == D3D_SIT_UAV_RWTYPED;
}

template<IRDescriptorRangeType DescriptorType>
static IRDescriptorRange1 CreateDescriptorRange(const uint32 NumDescriptors)
{
	IRDescriptorRange1 DescRange;
	DescRange.RangeType = DescriptorType;
	DescRange.NumDescriptors = NumDescriptors;
	DescRange.BaseShaderRegister = 0;
	DescRange.RegisterSpace = 0;
	DescRange.OffsetInDescriptorsFromTableStart = IRDescriptorRangeOffsetAppend;

	switch (DescriptorType)
	{
	case IRDescriptorRangeTypeCBV:
	case IRDescriptorRangeTypeSRV:
		DescRange.Flags = IRDescriptorRangeFlagDataStaticWhileSetAtExecute;
		break;
	case IRDescriptorRangeTypeUAV:
		DescRange.Flags = IRDescriptorRangeFlagDataVolatile;
		break;
	case IRDescriptorRangeTypeSampler:
		DescRange.Flags = IRDescriptorRangeFlagNone;
		break;
	default:
		checkNoEntry();
		break;
	}

	return DescRange;
}

static bool ProcessReflection(
	ID3D12ShaderReflection* ShaderReflection,
	const uint32 BoundResources,
	const FShaderCompilerInput& Input,
	const FShaderParameterParser& ShaderParameterParser,
	FShaderCompilerOutput& Output,
	CrossCompiler::FHlslccHeaderWriter& CCHeaderWriter,
	FMetalShaderOutputMetaData& OutputData,
	uint32& NumCBVs);

// Alias register space macros for backward compatibility
#define RAY_TRACING_REGISTER_SPACE_GLOBAL UE_HLSL_SPACE_RAY_TRACING_GLOBAL
#define RAY_TRACING_REGISTER_SPACE_LOCAL UE_HLSL_SPACE_RAY_TRACING_LOCAL
#define RAY_TRACING_REGISTER_SPACE_SYSTEM UE_HLSL_SPACE_RAY_TRACING_SYSTEM

inline void AppendRayTracingSystemParams(TArray<IRRootParameter1>& SignatureParams)
{
	IRRootParameter1 RootParam;
	RootParam.ShaderVisibility = IRShaderVisibilityAll;
	RootParam.ParameterType = IRRootParameterType32BitConstants;
	RootParam.Constants.Num32BitValues = 6;
	RootParam.Constants.RegisterSpace = RAY_TRACING_REGISTER_SPACE_SYSTEM;
	RootParam.Constants.ShaderRegister = RAY_TRACING_SYSTEM_ROOTCONSTANT_REGISTER;
	SignatureParams.Add(RootParam);
}

inline uint32 GetAutoBindingSpace(EShaderFrequency ShaderFrequency)
{
	switch (ShaderFrequency)
	{
	case SF_RayGen:
		return RAY_TRACING_REGISTER_SPACE_GLOBAL;
	case SF_RayMiss:
	case SF_RayHitGroup:
	case SF_RayCallable:
		return RAY_TRACING_REGISTER_SPACE_LOCAL;
	default:
		return 0;
	}
}

template <class T>
static bool ProcessReflection(T* ShaderReflection,
							  const uint32 BoundResources,
							  const FShaderCompilerInput& Input,
							  const FShaderParameterParser& ShaderParameterParser,
							  FShaderCompilerOutput& Output,
							  CrossCompiler::FHlslccHeaderWriter& CCHeaderWriter,
							  FMetalShaderOutputMetaData& OutputData,
							  FMetalBindlessShaderInfo& ShaderInfo,
							  IRShaderVisibility ShaderVisibility)
{
	uint32 NumSRVs = 0;
	uint32 NumUAVs = 0;
	uint32 NumSamplers = 0;

	int32 AutoBindingSpace = GetAutoBindingSpace(Input.Target.GetFrequency());
	
	// Build output metadata and collect infos for each resource type ranges.
	for (uint32 ResourceIndex = 0; ResourceIndex < BoundResources; ResourceIndex++)
	{
		D3D12_SHADER_INPUT_BIND_DESC BindDesc;
		ShaderReflection->GetResourceBindingDesc(ResourceIndex, &BindDesc);

		const FString BindDescName(BindDesc.Name);

		IRResourceType ResourceType = QuantizeD3DResourceType(BindDesc.Type);
		bool bIsResourceTyped = IsD3DResourceTypeTyped(BindDesc.Type);
		const uint32 BindIndex = BindDesc.BindPoint;

		// Reflect bindings for the input space only.
		if (BindDesc.Space != AutoBindingSpace)
		{
			if (ResourceType == IRResourceTypeCBV)
			{
				// Edge case: it should be fine to have the Globals CBV at (b0, space0); as the current implementation for loose globals binding (via ShaderParameterCollection) assumes this is the default binding point for bindless handles too.
				bool bGlobalCB = (FCStringAnsi::Strcmp(BindDesc.Name, "$Globals") == 0);
				if (!bGlobalCB)
				{
					continue;
				}
				else
				{
					// Check that Globals only contains bindless handles.
					ID3D12ShaderReflectionConstantBuffer* ConstantBuffer = ShaderReflection->GetConstantBufferByName(BindDesc.Name);

					D3D12_SHADER_BUFFER_DESC CBDesc;
					ConstantBuffer->GetDesc(&CBDesc);

					TArray<D3D12_SHADER_VARIABLE_DESC> LooseGlobalsFound;
					
					// Track all of the variables in this constant buffer.
					for (uint32 ConstantIndex = 0; ConstantIndex < CBDesc.Variables; ConstantIndex++)
					{
						ID3D12ShaderReflectionVariable* Variable = ConstantBuffer->GetVariableByIndex(ConstantIndex);
						
						D3D12_SHADER_VARIABLE_DESC VariableDesc;
						Variable->GetDesc(&VariableDesc);
						
						if (VariableDesc.uFlags & D3D_SVF_USED)
						{
							if (FCStringAnsi::Strstr(VariableDesc.Name, "BindlessResource_") == nullptr && FCStringAnsi::Strstr(VariableDesc.Name, "BindlessSampler_") == nullptr)
							{
								LooseGlobalsFound.Add(VariableDesc);
							}
						}
					}
					
					bool bOnlyContainsBindlessHandle = LooseGlobalsFound.IsEmpty();
					if (!bOnlyContainsBindlessHandle)
					{
						FShaderCompilerError LooseGlobalsError(TEXT("Error: $Globals contains loose globals! Make sure that your shader pipeline is correctly setup, and all the loose globals are moved to the Root constant buffer. Loose globals found:"));
						Output.Errors.Add(LooseGlobalsError);
						
						for (D3D12_SHADER_VARIABLE_DESC const& LooseGlobal : LooseGlobalsFound)
						{
							FShaderCompilerError Error(FString::Printf( TEXT("\t- %s (offset: %u size: %u)"), ANSI_TO_TCHAR(LooseGlobal.Name), LooseGlobal.StartOffset, LooseGlobal.Size));
							Output.Errors.Add(Error);
						}
						Output.bSucceeded = false;
						
						return false;
					}
				}
			}
			else
			{
				continue;
			}
		}
		
		switch (ResourceType)
		{
		case IRResourceTypeSRV:
			if (bIsResourceTyped)
				OutputData.TypedBuffers |= (1 << BindIndex);
			else
				OutputData.InvariantBuffers |= (1 << BindIndex);

			CCHeaderWriter.WriteSRV(*BindDescName, BindIndex, BindDesc.BindCount);
			NumSRVs = FMath::Max(NumSRVs, BindIndex + BindDesc.BindCount);
			break;

		case IRResourceTypeUAV:
			if (bIsResourceTyped)
			{
				OutputData.TypedUAVs |= (1 << BindIndex);
				OutputData.TypedBuffers |= (1 << BindIndex);
			}
			else
			{
				OutputData.InvariantBuffers |= (1 << BindIndex);
			}

			CCHeaderWriter.WriteUAV(*BindDescName, BindIndex, BindDesc.BindCount);
			NumUAVs = FMath::Max(NumUAVs, BindIndex + BindDesc.BindCount);
			break;

		case IRResourceTypeSampler:
			CCHeaderWriter.WriteSamplerState(*BindDescName, BindIndex);
			NumSamplers = FMath::Max(NumSamplers, BindIndex + BindDesc.BindCount);
			break;

		case IRResourceTypeCBV:
		{
			const bool bIsGlobalCB = (FCStringAnsi::Strcmp(BindDesc.Name, "$Globals") == 0);
			const bool bIsRootCB = (BindDescName == FShaderParametersMetadata::kRootUniformBufferBindingName);

			int32 ConstantBufferSize = 0;
			
			OutputData.ConstantBuffers |= (1 << BindIndex);
			
			IRRootParameter1 RootParam;
			RootParam.ParameterType = IRRootParameterTypeCBV;
			RootParam.ShaderVisibility = ShaderVisibility;
			RootParam.Descriptor.ShaderRegister = BindIndex;
			RootParam.Descriptor.RegisterSpace = BindDesc.Space;
			RootParam.Descriptor.Flags = IRRootDescriptorFlagDataStaticWhileSetAtExecute;

			TMap<uint32, FString>& ShaderSpaceRootParams = ShaderInfo.RootParamsCache[BindDesc.Space];
			FString* It = ShaderSpaceRootParams.Find(BindIndex);
			if (!It)
			{
				if (Input.IsRayTracingShader() && BindDesc.Space == RAY_TRACING_REGISTER_SPACE_LOCAL)
				{
					ShaderInfo.UnsortedLocalRootParams.Add(RootParam);
				}
				else
				{
					ShaderInfo.GlobalRootParams.Add(RootParam);
				}

				ShaderSpaceRootParams.Add(BindIndex, FString(BindDesc.Name));
			}
			else
			{
				continue;
			}
			
			// Global uniform buffer - handled specially as we care about the internal layout
			if (bIsGlobalCB || bIsRootCB)
			{
				TCBDMARangeMap CBRanges;
				CCHeaderWriter.WritePackedUB(BindIndex);

				ID3D12ShaderReflectionConstantBuffer* ConstantBuffer = ShaderReflection->GetConstantBufferByName(BindDesc.Name);

				D3D12_SHADER_BUFFER_DESC CBDesc;
				ConstantBuffer->GetDesc(&CBDesc);

				// Track all of the variables in this constant buffer.
				for (uint32 ConstantIndex = 0; ConstantIndex < CBDesc.Variables; ConstantIndex++)
				{
					ID3D12ShaderReflectionVariable* Variable = ConstantBuffer->GetVariableByIndex(ConstantIndex);

					D3D12_SHADER_VARIABLE_DESC VariableDesc{};
					Variable->GetDesc(&VariableDesc);

					if (VariableDesc.uFlags & D3D_SVF_USED)
					{
						const FString VariableName(VariableDesc.Name);
		
						// Bug in DXC reflection, collapses an array-of-matrices to a single matrix in one specific
						// case: when the array length equals the matrix component count (Rows*Cols), e.g.
						// float4x4[16].
						uint32 VariableSize = VariableDesc.Size;
						{
							D3D12_SHADER_TYPE_DESC TypeDesc{};
							if (ID3D12ShaderReflectionType* VarType = Variable->GetType())
							{
								VarType->GetDesc(&TypeDesc);
							}

							if (TypeDesc.Class == D3D_SVC_MATRIX_ROWS || TypeDesc.Class == D3D_SVC_MATRIX_COLUMNS)
							{
								if (const FShaderParameterParser::FParsedShaderParameter* Parsed = ShaderParameterParser.FindParameterInfosUnsafe(VariableName))
								{
									if (!Parsed->ParsedArraySize.IsEmpty())
									{
										const int32 ArrayCount = FCString::Atoi(*FString(Parsed->ParsedArraySize));
										const uint32 MatrixComponentCount = TypeDesc.Rows * TypeDesc.Columns;

										// Only fires for the DXC misreport: parsed array length == Rows*Cols.
										if (ArrayCount > 1 && (uint32)ArrayCount == MatrixComponentCount)
										{
											const uint32 StorageRows = (TypeDesc.Class == D3D_SVC_MATRIX_ROWS) ? TypeDesc.Rows : TypeDesc.Columns;
											const uint32 ElementStride = StorageRows * 16u;
											VariableSize = FMath::Max<uint32>(VariableDesc.Size, ElementStride * (uint32)ArrayCount);
										}
									}
								}
							}
						}

						CCHeaderWriter.WritePackedUBField(*VariableName, VariableDesc.StartOffset, VariableSize);

						const uint32 MbrOffset = VariableDesc.StartOffset / sizeof(float);
						const uint32 MbrSize = VariableSize / sizeof(float);
						unsigned DestCBPrecision = TEXT('h');
						unsigned SourceOffset = MbrOffset;
						unsigned DestOffset = MbrOffset;
						unsigned DestSize = MbrSize;
						unsigned DestCBIndex = 0;
						InsertRange(CBRanges, BindIndex, SourceOffset, DestSize, DestCBIndex, DestCBPrecision, DestOffset);

						if(bIsRootCB)
						{
							HandleReflectedRootConstantBufferMember(
								Input,
								ShaderParameterParser,
								FString(VariableDesc.Name),
								VariableDesc.StartOffset,
								VariableSize,
								Output);
						}
						else
						{
							HandleReflectedGlobalConstantBufferMember(
								Input,
								ShaderParameterParser,
								VariableName,
								BindIndex,
								VariableDesc.StartOffset,
								VariableSize,
								Output);
						}

						ConstantBufferSize = FMath::Max<int32>(ConstantBufferSize, VariableDesc.StartOffset + VariableSize);
					}
				}
				
				if (bIsRootCB)
				{
					if (ConstantBufferSize > 0)
					{
						HandleReflectedRootConstantBuffer(ConstantBufferSize, Output);
					}
				}
			}
			else
			{
				ID3D12ShaderReflectionConstantBuffer* ConstantBuffer = ShaderReflection->GetConstantBufferByName(BindDesc.Name);

				D3D12_SHADER_BUFFER_DESC CBDesc{};
				ConstantBuffer->GetDesc(&CBDesc);

				const FString UniformBufferName(BindDesc.Name);

				const EUniformBufferMemberReflectionReason Reason = ShouldReflectUniformBufferMembers(Input, UniformBufferName);
				if (Reason != EUniformBufferMemberReflectionReason::None)
				{
					for (uint32 ConstantIndex = 0; ConstantIndex < CBDesc.Variables; ConstantIndex++)
					{
						ID3D12ShaderReflectionVariable* Variable = ConstantBuffer->GetVariableByIndex(ConstantIndex);

						D3D12_SHADER_VARIABLE_DESC VariableDesc;
						Variable->GetDesc(&VariableDesc);

						if (VariableDesc.uFlags & D3D_SVF_USED)
						{
							const FString VariableName(VariableDesc.Name);

							HandleReflectedUniformBufferConstantBufferMember(
								Input,
								Reason,
								UniformBufferName,
								BindIndex,
								VariableName,
								VariableDesc.StartOffset,
								VariableDesc.Size,
								Output
							);
						}
					}
				}
				
				// Regular uniform buffer - we only care about the binding index
				CCHeaderWriter.WriteUniformBlock(*UniformBufferName, BindIndex);
				HandleReflectedUniformBuffer(Input, UniformBufferName, BindIndex, Output);
			}
			ShaderInfo.NumCBVs = FMath::Max(ShaderInfo.NumCBVs, BindIndex + BindDesc.BindCount);
		}
		break;
		default:
			checkNoEntry();
		};
	}

	// DXIL fetches resources from the resources heaps.
	check(NumSRVs == 0 && NumUAVs == 0);
	
	return true;
}

#if PLATFORM_WINDOWS
static dxc::DxcDllSupport& GetDxcDllHelper()
{
	struct DxcDllHelper
	{
		DxcDllHelper()
		{
			const HRESULT Result = DxcDllSupport.Initialize();
			if (FAILED(Result))
			{
				//TODO: Do something
			}
		}
		dxc::DxcDllSupport DxcDllSupport;
	};

	static DxcDllHelper DllHelper;
	return DllHelper.DxcDllSupport;
}
#endif // PLATFORM_WINDOWS

static bool ReflectDXILAndBuildDescriptorRanges(const TArray<uint32>& DXILReflection,
												const FShaderCompilerInput& Input,
												const FShaderParameterParser& ShaderParameterParser,
												FShaderCompilerOutput& Output,
												CrossCompiler::FHlslccHeaderWriter& CCHeaderWriter,
												FMetalShaderOutputMetaData& OutputData,
												FMetalBindlessShaderInfo& ShaderInfo,
												IRShaderVisibility ShaderVisibility)
{
	// Reflect DXIL
	TRefCountPtr<IDxcUtils> Utils;
#if PLATFORM_MAC
	HRESULT Result = DxcCreateInstance(CLSID_DxcUtils, IID_PPV_ARGS(Utils.GetInitReference()));
#elif PLATFORM_WINDOWS
	dxc::DxcDllSupport& DxcDllHelper = GetDxcDllHelper();
	HRESULT Result = DxcDllHelper.CreateInstance2(UE::ShaderCompilerCommon::GetDxcMalloc(), CLSID_DxcUtils, Utils.GetInitReference());
#endif
	if (!SUCCEEDED(Result))
	{
		UE_LOGF(LogShaders, Warning, "Failed to create DxcUtils");
		return false;
	}

	DxcBuffer ReflBuffer = {0};
	ReflBuffer.Ptr = DXILReflection.GetData();
	ReflBuffer.Size = DXILReflection.Num() * sizeof(uint32_t);
	
	// Stolen from D3DShaderCompilerDXC (do we really need this for Metal?)
	uint32 ShaderRequiresFlags = 0;

	if (!Input.IsRayTracingShader())
	{
		TRefCountPtr<ID3D12ShaderReflection> ShaderReflection;
		Result = Utils->CreateReflection(&ReflBuffer, IID_PPV_ARGS(ShaderReflection.GetInitReference()));
		if (!SUCCEEDED(Result))
		{
			UE_LOGF(LogShaders, Warning, "Failed to create shader reflection (CreateReflection returned 0x%x)", Result);
			return false;
		}

		D3D12_SHADER_DESC ShaderDesc = {};
		ShaderReflection->GetDesc(&ShaderDesc);
		
		// TODO: Carl - Fix this
		ShaderInfo.NumInstructions = ShaderDesc.InstructionCount;
		
		// Return a fraction of the number of instructions as DXIL is more verbose than DXBC.
		// Ratio 119:307 was estimated by gathering average instruction count for D3D11 and D3D12 shaders in ShooterGame with result being ~ 357:921.
		constexpr uint32 DxbcToDxilInstructionRatio[2] = { 119, 307 };
		ShaderInfo.NumInstructions = ShaderInfo.NumInstructions * DxbcToDxilInstructionRatio[0] / DxbcToDxilInstructionRatio[1];
		
		if (!ProcessReflection<ID3D12ShaderReflection>(ShaderReflection.GetReference(),
												  ShaderDesc.BoundResources,
												  Input, 
												  ShaderParameterParser,
												  Output,
												  CCHeaderWriter,
												  OutputData,
												  ShaderInfo,
												  ShaderVisibility))
		{
			return false;
		}

		// Vertex Input
		for (uint32 InputIndex = 0; InputIndex < ShaderDesc.InputParameters; InputIndex++)
		{
			D3D12_SIGNATURE_PARAMETER_DESC SignatureParamDesc;
			ShaderReflection->GetInputParameterDesc(InputIndex, &SignatureParamDesc);

			FString TypeQualifier;
			switch (SignatureParamDesc.ComponentType)
			{
			case D3D_REGISTER_COMPONENT_UINT32:
				TypeQualifier = TEXT("u");
				break;
			case D3D_REGISTER_COMPONENT_SINT32:
				TypeQualifier = TEXT("i");
				break;
			case D3D_REGISTER_COMPONENT_FLOAT32:
				TypeQualifier = TEXT("f");
				break;
			case D3D_REGISTER_COMPONENT_UNKNOWN:
			default:
				checkNoEntry();
				break;
			}

			CCHeaderWriter.WriteInputAttribute(TEXT("in_ATTRIBUTE"), *TypeQualifier, SignatureParamDesc.SemanticIndex, /*bLocationPrefix:*/ false, /*bLocationSuffix:*/ true);
		}

		// Pixel Output
		for (uint32 OutputIndex = 0; OutputIndex < ShaderDesc.OutputParameters; OutputIndex++)
		{
			D3D12_SIGNATURE_PARAMETER_DESC SignatureParamDesc;
			ShaderReflection->GetOutputParameterDesc(OutputIndex, &SignatureParamDesc);

			FString TypeQualifier;
			switch (SignatureParamDesc.ComponentType)
			{
			case D3D_REGISTER_COMPONENT_UINT32:
				TypeQualifier = TEXT("u");
				break;
			case D3D_REGISTER_COMPONENT_SINT32:
				TypeQualifier = TEXT("i");
				break;
			case D3D_REGISTER_COMPONENT_FLOAT32:
				TypeQualifier = TEXT("f");
				break;
			case D3D_REGISTER_COMPONENT_UNKNOWN:
			default:
				checkNoEntry();
				break;
			}

			FString SemanticName = SignatureParamDesc.SemanticName;
			CCHeaderWriter.WriteOutputAttribute(*SemanticName, *TypeQualifier, SignatureParamDesc.SemanticIndex, /*bLocationPrefix:*/ false, /*bLocationSuffix:*/ true);
		}
	}
	else
	{
		TRefCountPtr<ID3D12LibraryReflection> LibraryReflection;
		Result = Utils->CreateReflection(&ReflBuffer, IID_PPV_ARGS(LibraryReflection.GetInitReference()));
		if (!SUCCEEDED(Result))
		{
			UE_LOGF(LogShaders, Error, "Failed to create library reflection (CreateReflection returned 0x%x)", Result);
			return false;
		}
		
		D3D12_LIBRARY_DESC LibraryDesc = {};
		LibraryReflection->GetDesc(&LibraryDesc);

		// MangledEntryPoints contains partial mangled entry point signatures in a the following form:
		// ?QualifiedName@ (as described here: https://en.wikipedia.org/wiki/Name_mangling)
		// Entry point parameters are currently not included in the partial mangling.
		TArray<FString, TInlineAllocator<3>> MangledEntryPoints;
		UE::ShaderCompilerCommon::ParseRayTracingEntryPoint(Input.EntryPointName, ShaderInfo.RayEntryPoint, ShaderInfo.RayAnyHitEntryPoint, ShaderInfo.RayIntersectionEntryPoint);
		
		if (!ShaderInfo.RayEntryPoint.IsEmpty())
		{
			MangledEntryPoints.Add(FString::Printf(TEXT("?%s@"), *ShaderInfo.RayEntryPoint));
		}
		if (!ShaderInfo.RayIntersectionEntryPoint.IsEmpty())
		{
			MangledEntryPoints.Add(FString::Printf(TEXT("?%s@"), *ShaderInfo.RayIntersectionEntryPoint));
			ShaderInfo.bIsProcedural = true;
		}
		if (!ShaderInfo.RayAnyHitEntryPoint.IsEmpty())
		{
			MangledEntryPoints.Add(FString::Printf(TEXT("?%s@"), *ShaderInfo.RayAnyHitEntryPoint));
		}
		
		uint32 NumFoundEntryPoints = 0;

		for (uint32 FunctionIndex = 0; FunctionIndex < LibraryDesc.FunctionCount; ++FunctionIndex)
		{
			ID3D12FunctionReflection* FunctionReflection = LibraryReflection->GetFunctionByIndex(FunctionIndex);
			
			D3D12_FUNCTION_DESC FunctionDesc = {};
			Result = FunctionReflection->GetDesc(&FunctionDesc);
			if (!SUCCEEDED(Result))
			{
				UE_LOGF(LogShaders, Error, "Failed to get function reflection (ID3D12FunctionReflection::GetDesc returned 0x%x)", Result);
				return false;
			}
			
			for (const FString& MangledEntryPoint : MangledEntryPoints)
			{
				// Entry point parameters are currently not included in the partial mangling, therefore partial substring match is used here.
				if (FCStringAnsi::Strstr(FunctionDesc.Name, TCHAR_TO_ANSI(*MangledEntryPoint)))
				{
					ProcessReflection<ID3D12FunctionReflection>(FunctionReflection,
																FunctionDesc.BoundResources,
																Input,
																ShaderParameterParser,
																Output,
																CCHeaderWriter, 
																OutputData,
																ShaderInfo, 
																ShaderVisibility);
					NumFoundEntryPoints++;
				}
			}
			ShaderInfo.NumInstructions = FunctionDesc.InstructionCount;
		}

		if (NumFoundEntryPoints != MangledEntryPoints.Num())
		{
			UE_LOGF(LogShaders, Error, "Failed to find required points in the shader library.");
			return false;
		}
	}

	return true;
}

struct FMetalShaderParameterParserPlatformConfiguration : public FShaderParameterParser::FPlatformConfiguration
{
	FMetalShaderParameterParserPlatformConfiguration(const FShaderCompilerInput& Input)
		: FShaderParameterParser::FPlatformConfiguration(TEXTVIEW("cbuffer"), EShaderParameterParserConfigurationFlags::UseStableConstantBuffer|EShaderParameterParserConfigurationFlags::SupportsBindless)
		, bIsRayTracingShader(Input.IsRayTracingShader())
		, HitGroupSystemIndexBufferName(FShaderParameterParser::kBindlessSRVPrefix + FString(TEXT("HitGroupSystemIndexBuffer")))
		, HitGroupSystemVertexBufferName(FShaderParameterParser::kBindlessSRVPrefix + FString(TEXT("HitGroupSystemVertexBuffer")))
	{}

	virtual FString GenerateBindlessAccess(EBindlessConversionType BindlessType, FStringView FullTypeString, FStringView ArrayNameOverride, FStringView IndexString) const final
	{
		const TCHAR* HeapString = BindlessType == EBindlessConversionType::Sampler ? TEXT("SamplerDescriptorHeap") : TEXT("ResourceDescriptorHeap");

		if (bIsRayTracingShader)
		{
			if (BindlessType == EBindlessConversionType::SRV)
			{
				// Patch the HitGroupSystemIndexBuffer/HitGroupSystemVertexBuffer indices to use the ones contained in the shader record
				if (IndexString == HitGroupSystemIndexBufferName)
				{
					IndexString = TEXTVIEW("MetalHitGroupSystemParameters.BindlessHitGroupSystemIndexBuffer");
				}
				else if (IndexString == HitGroupSystemVertexBufferName)
				{
					IndexString = TEXTVIEW("MetalHitGroupSystemParameters.BindlessHitGroupSystemVertexBuffer");
				}
			}

			// Raytracing shaders need NonUniformResourceIndex because bindless index can be divergent in hit/miss/callable shaders
			return FString::Printf(TEXT("%s[NonUniformResourceIndex(%.*s)]"),
				HeapString,
				IndexString.Len(), IndexString.GetData()
			);
		}
		
		return FString::Printf(TEXT("%s[%.*s]"),
			HeapString,
			IndexString.Len(), IndexString.GetData()
		);
	}
	
	bool bIsRayTracingShader = false;
	const FString HitGroupSystemIndexBufferName;
	const FString HitGroupSystemVertexBufferName;
};

extern bool ShouldStripIndividualMetalLib(const FShaderCompilerInput& InputCompilerEnvironment);

void CompileToMetalLib(FMetalShaderBytecode& InMetalBytecode,
					   IRShaderStage& ShaderStageOut,
					   const FShaderCompilerInput& Input,
					   FShaderCompilerOutput& Output,
					   const char*& ReflectionJSON,
					   bool bDumpDebugInfo,
					   bool& bUsesDiscard,
					   uint32& OutputSizeVS,
					   uint32& MaxInputPrimitivesPerMeshThreadgroupGS,
					   CrossCompiler::FHlslccHeaderWriter& CCHeaderWriter,
					   auto CompileCallback,
					   FString* OutActualEntryPoint = nullptr)
{
	// Uncomment to enable IR validation.
	//IRCompilerSetValidationFlags(CompilerInstance, IRCompilerValidationFlagAll);
	
	IRError* CompileError = nullptr;
	IRObject* AirBytecode = CompileCallback(&CompileError);
	ON_SCOPE_EXIT
	{
		if (AirBytecode)
		{
			IRObjectDestroy(AirBytecode);
		}
	};
	
	if (!AirBytecode || CompileError != nullptr)
	{
		Output.AddErrorf(TEXT("Error: MetalShaderConverter failed to produce air bytecode for '%s' (%s)!"), *Input.EntryPointName, (CompileError ? ANSI_TO_TCHAR((const char*)IRErrorGetPayload(CompileError)) : TEXT("CompileError NULL")));
		Output.bSucceeded = false;
		
		return;
	}
	ShaderStageOut = IRObjectGetMetalIRShaderStage(AirBytecode);

	IRShaderReflection* AirReflection = IRShaderReflectionCreate();
	IRObjectGetReflection(AirBytecode, ShaderStageOut, AirReflection);

	// RT-only: MSC (LLVM) appends ".N" when the requested entry-point name collides with an
	// already-linked symbol. Capture the emitted name so the header binds the real symbol.
	if (OutActualEntryPoint)
	{
		if (const char* Emitted = IRShaderReflectionGetEntryPointFunctionName(AirReflection))
		{
			*OutActualEntryPoint = ANSI_TO_TCHAR(Emitted);
		}
	}
	
	if(bDumpDebugInfo)
	{
		ReflectionJSON = IRShaderReflectionCopyJSONString(AirReflection);
		if(ReflectionJSON)
		{
			FString ReflectionString = ANSI_TO_TCHAR(ReflectionJSON);
			DumpDebugShaderText(Input, ReflectionString, TEXT("reflection.json"));
		}
	}
	
	switch (ShaderStageOut)
	{
		case IRShaderStageVertex:
		{
			// Retrieve VS infos only if GS emulation is used (VS output size is useless otherwise).
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
			IRVersionedVSInfo Info;
			bool bSuccessfulReflectionVS = IRShaderReflectionCopyVertexInfo(AirReflection, IRReflectionVersion_1_0, &Info);
			check(bSuccessfulReflectionVS);
			
			OutputSizeVS = Info.info_1_0.vertex_output_size_in_bytes;
			
			IRShaderReflectionReleaseVertexInfo(&Info);
#endif
			
			if(!ReflectionJSON)
			{
				// Serialize Reflection for vs (required to generate stage_in functions at PSO creation-time)
				ReflectionJSON = IRShaderReflectionCopyJSONString(AirReflection);
				checkSlow(ReflectionJSON);
			}
			break;
		}
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
		case IRShaderStageGeometry:
		{
			IRVersionedGSInfo Info;
			bool bSuccessfulReflectionGS = IRShaderReflectionCopyGeometryInfo(AirReflection, IRReflectionVersion_1_0, &Info);
			check(bSuccessfulReflectionGS);
			
			MaxInputPrimitivesPerMeshThreadgroupGS = Info.info_1_0.max_input_primitives_per_mesh_threadgroup;
			
			IRShaderReflectionReleaseGeometryInfo(&Info);
			break;
		}
#endif
		case IRShaderStageFragment:
		{
			IRVersionedFSInfo Info;
			bool bSuccessfulReflectionPS = IRShaderReflectionCopyFragmentInfo(AirReflection, IRReflectionVersion_1_0, &Info);
			check(bSuccessfulReflectionPS);
			
			bUsesDiscard = Info.info_1_0.discards;
			
			IRShaderReflectionReleaseFragmentInfo(&Info);
			break;
		}	
		case IRShaderStageCompute:
		{
			IRVersionedCSInfo Info;
			bool bSuccessfulReflectionCS = IRShaderReflectionCopyComputeInfo(AirReflection, IRReflectionVersion_1_0, &Info);
			check(bSuccessfulReflectionCS);
			
			CCHeaderWriter.WriteNumThreads(Info.info_1_0.tg_size[0], Info.info_1_0.tg_size[1], Info.info_1_0.tg_size[2]);
			
			IRShaderReflectionReleaseComputeInfo(&Info);
			break;
		}
		default:
			break;
	}
	IRShaderReflectionDestroy(AirReflection);
	
	// Retrieve the generated .metallib
	IRMetalLibBinary* GeneratedMetalLib = IRMetalLibBinaryCreate();
	ON_SCOPE_EXIT
	{
		IRMetalLibBinaryDestroy(GeneratedMetalLib);
	};
	if (!IRObjectGetMetalLibBinary(AirBytecode, ShaderStageOut, GeneratedMetalLib))
	{
		Output.AddErrorf(TEXT("Error: MetalShaderConverter failed to produce a metallib for '%s'!"), *Input.EntryPointName);
		Output.bSucceeded = false;
		
		return;
	}
	
	size_t MetalLibSize = IRMetalLibGetBytecodeSize(GeneratedMetalLib);
	InMetalBytecode.OutputFile.SetNum(MetalLibSize);
	size_t OutMetalLibSize = IRMetalLibGetBytecode(GeneratedMetalLib, reinterpret_cast<uint8_t*>(InMetalBytecode.OutputFile.GetData()));
	
	const bool bUseNativeShaderLibrary = Input.Environment.CompilerFlags.Contains(CFLAG_Archive);
	if (bUseNativeShaderLibrary)
	{
		// Copy the AIR (needed for native shader library)
		InMetalBytecode.ObjectFile = InMetalBytecode.OutputFile;
	}
	else if (ShouldStripIndividualMetalLib(Input))
	{
		// For non-native shader library we need to strip the metallib now
		// First serialize the metallib to disk and then strip it
		const FString& TempDir = FMetalCompilerToolchain::Get()->GetLocalTempDir();
		const FString MetalLibFilePath = FPaths::CreateTempFilename(*TempDir, TEXT("MSCLibrary-"), *FMetalCompilerToolchain::MetalLibraryExtension);
		
		ON_SCOPE_EXIT
		{
			// Clean up the .metallib file
			IFileManager::Get().Delete(*MetalLibFilePath);
		};
		
		// Serialize it back to the OutputFile
		if (!FFileHelper::SaveArrayToFile(InMetalBytecode.OutputFile, *MetalLibFilePath))
		{
			Output.AddErrorf(TEXT("Error: MetalShaderConverter failed to write a metallib '%s'!"), *MetalLibFilePath);
			Output.bSucceeded = false;

			return;
		}
		
		if (!FMetalCompilerToolchain::Get()->StripMetalLib(AppleSDKMac, MetalLibFilePath, MetalLibFilePath, false))
		{
			Output.AddErrorf(TEXT("Error: MetalShaderConverter failed to strip a metallib '%s'!"), *MetalLibFilePath);
			Output.bSucceeded = false;

			return;
		}
		
		// Serialize it back to the OutputFile
		if (!FFileHelper::LoadFileToArray(InMetalBytecode.OutputFile, *MetalLibFilePath))
		{
			Output.AddErrorf(TEXT("Error: MetalShaderConverter failed to read a metallib '%s'!"), *MetalLibFilePath);
			Output.bSucceeded = false;

			return;
		}
	}
	
	Output.bSucceeded = true;
};

void FMetalCompileShaderMSC::DoCompileMetalShader(
	const FShaderCompilerInput& Input,
	const FShaderParameterParser& InShaderParameterParser, // todo: why isn't this one used?
	FShaderCompilerOutput& Output,
	const FString& InPreprocessedShader,
	uint32 VersionEnum,
	EMetalGPUSemantics Semantics,
	uint32 MaxUnrollLoops,
	EShaderFrequency Frequency,
	bool bDumpDebugInfo,
	const FString& Standard,
	const FString& MinOSVersion,
	FMetalBindlessShaderInfo& ShaderInfo)
{
	int32 IABTier = VersionEnum >= 4 ? Input.Environment.GetCompileArgument(TEXT("METAL_INDIRECT_ARGUMENT_BUFFERS"), 0) : 0;

	Output.bSucceeded = false;

	std::string MetalSource;
	FString MetalErrors;
	
	bool const bZeroInitialise = Input.Environment.CompilerFlags.Contains(CFLAG_ZeroInitialise);
	bool const bBoundsChecks = Input.Environment.CompilerFlags.Contains(CFLAG_BoundsChecking);

	bool bAllowFastIntrinsics = true;

	// WPO requires that we make all multiply/sincos instructions invariant :(
	bool bForceInvariance = Input.Environment.GetCompileArgument(TEXT("USES_WORLD_POSITION_OFFSET"), false);
	
	FMetalShaderOutputMetaData OutputData;
	
	struct FMetalResourceTableEntry : FUniformResourceEntry
	{
		FString Name;
		uint32 Size;
		uint32 SetIndex;
		bool bUsed;
	};
	TMap<FString, TArray<FMetalResourceTableEntry>> IABs;

	FString PreprocessedShader = InPreprocessedShader;
	
	const char* ReflectionJSON = nullptr;
	bool bUsesDiscard = false;
	uint32 OutputSizeVS = 0;
	uint32 MaxInputPrimitivesPerMeshThreadgroupGS = 0;
	
#if PLATFORM_MAC || PLATFORM_WINDOWS
	FMetalShaderParameterParserPlatformConfiguration PlatformConfiguration(Input);
	FShaderParameterParser ShaderParameterParser(PlatformConfiguration);
	if (!ShaderParameterParser.ParseAndModify(Input, Output.Errors, PreprocessedShader))
	{
		// The FShaderParameterParser will add any relevant errors.
		return;
	}

	{
		std::string EntryPointNameAnsi(TCHAR_TO_UTF8(*Input.EntryPointName));

		CrossCompiler::FShaderConductorContext CompilerContext;

		// Initialize compilation options for ShaderConductor
		CrossCompiler::FShaderConductorOptions Options(Input.Environment.CompilerFlags);
		Options.bEnable16bitTypes = true;
		// Make sure int64 atomics and dynamic heap indexing are available.
		Options.ShaderModel = {6, 6};
		
		if (Input.Environment.CompilerFlags.Contains(CFLAG_HLSL2021))
		{
			Options.HlslVersion = 2021;
		}
		
		
		if (Frequency == SF_Pixel)
		{
			ShaderInfo.bUsesDualSourceBlending = (PreprocessedShader.Find(TEXT("[[vk::location(0), vk::index(")) != INDEX_NONE);
		}
		
		TArray<FString> ExtraArgs;
		
		if (Input.Environment.CompilerFlags.Contains(CFLAG_GenerateSymbols))
		{
			ExtraArgs.Add(TEXT("-Zi"));
			ExtraArgs.Add(TEXT("-Qembed_debug"));
			ExtraArgs.Add(TEXT("--ignore-line-directives"));
		}
		else if (Input.IsRayTracingShader()) // This is required for RT shaders pending a discussion with Apple
		{
			ExtraArgs.Add(TEXT("-Zi"));
			ExtraArgs.Add(TEXT("-Qembed_debug"));
		}
		
		if (Input.Environment.CompilerFlags.Contains(CFLAG_Debug) || Input.Environment.CompilerFlags.Contains(CFLAG_SkipOptimizationsDXC))
		{
			// Currently cannot enable -Od because we have unbound parameters
			ExtraArgs.Add(TEXT("-O1"));
		}
		else if (Input.Environment.CompilerFlags.Contains(CFLAG_StandardOptimization))
		{
			ExtraArgs.Add(TEXT("-O1"));
		}
		else
		{
			ExtraArgs.Add(TEXT("-O3"));
		}
		
		const uint32 AutoBindingSpace = GetAutoBindingSpace(Input.Target.GetFrequency());
		{
			ExtraArgs.Add(TEXT("-auto-binding-space"));
			ExtraArgs.Add(FString::Printf(TEXT("%d"), AutoBindingSpace));
		}
		
		if (Input.Environment.CompilerFlags.Contains(CFLAG_PreferFlowControl))
		{
			ExtraArgs.Add(TEXT("-Gfp"));
		}
		
		if (Input.Environment.CompilerFlags.Contains(CFLAG_AvoidFlowControl))
		{
			ExtraArgs.Add(TEXT("-Gfa"));
		}
		
		// Load shader source into compiler context
		CompilerContext.LoadSource(PreprocessedShader, Input.VirtualSourceFilePath, Input.EntryPointName, Frequency, nullptr, &ExtraArgs);

		// Convert shader source to ANSI string
		FAnsiString SourceData = FAnsiString::ConstructFromPtrSize(CompilerContext.GetSourceString(), CompilerContext.GetSourceLength());

		// Replace special case texture "gl_LastFragData" by native subpass fetch operation
		static const uint32 MaxMetalSubpasses = 8;
		uint32 SubpassInputsDim[MaxMetalSubpasses];

		bool bSourceDataWasModified = PatchSpecialTextureInHlslSource(SourceData, SubpassInputsDim, MaxMetalSubpasses);
		
		// If source data was modified, reload it into the compiler context
		if (bSourceDataWasModified)
		{
			CompilerContext.LoadSource(SourceData, Input.VirtualSourceFilePath, Input.EntryPointName, Frequency, nullptr, &ExtraArgs);
		}

		if (bDumpDebugInfo)
		{
			DumpDebugShaderText(Input, &SourceData[0], SourceData.Len(), TEXT("rewritten.hlsl"));
		}
		
		CrossCompiler::FHlslccHeaderWriter CCHeaderWriter;

		FString ALNString;
		FString RTString;
		uint32 IABOffsetIndex = 0;
		uint64 BufferIndices = 0xffffffffffffffff;
		
		// Compile HLSL source to DXIL binary
		TArray<uint32> DxilData;
		
		if (!CompilerContext.CompileHlslToDxil(Options, DxilData))
		{
			UE_LOGF(LogShaders, Error, "Failed to produce DXIL bytecode for '%ls' '%ls'!", *Input.EntryPointName, *Input.DumpDebugInfoPath);
			CompilerContext.FlushErrors(Output.Errors);
			
			for (const FShaderCompilerError& Error : Output.Errors)
			{
				UE_LOGF(LogShaders, Error, "%ls", *Error.GetErrorStringWithLineMarker());
			}
			
			Output.bSucceeded = false;
			
			return;
		}
		
		// Return code reflection if requested for shader analysis
		if (Input.Environment.CompilerFlags.Contains(CFLAG_OutputAnalysisArtifacts))
		{
			FGenericShaderStat ShaderCodeReflection;
			if (CrossCompiler::FShaderConductorContext::Disassemble(CrossCompiler::EShaderConductorIR::Dxil, DxilData.GetData(), DxilData.Num()*sizeof(uint32), ShaderCodeReflection))
			{
				Output.ShaderStatistics.Add(MoveTemp(ShaderCodeReflection));
			}
		}
		
		if (bDumpDebugInfo)
		{
			DumpDebugShaderBinary(Input, DxilData.GetData(), DxilData.Num() * sizeof(uint32), TEXT("dxil"));
		}
		
		ANSICHAR MainCRC[25];
		ShaderInfo.CRCLen = DxilData.Num() * sizeof( uint32_t );
		ShaderInfo.CRC = FCrc::MemCrc_DEPRECATED(DxilData.GetData(), ShaderInfo.CRCLen);
		FCStringAnsi::Snprintf(MainCRC, 25, "Main_%0.8x_%0.8x", ShaderInfo.CRCLen, ShaderInfo.CRC);
		
		const IRShaderVisibility ShaderVisibility = ShaderFrequencyToVisibility(Frequency);
		
		// Build shader metadata and root signature parameters
		bool bSuccessfulReflection = ReflectDXILAndBuildDescriptorRanges(DxilData,
																		 Input,
																		 ShaderParameterParser,
																		 Output,
																		 CCHeaderWriter,
																		 OutputData,
																		 ShaderInfo,
																		 ShaderVisibility);
		check(bSuccessfulReflection);
		
		Output.NumInstructions = ShaderInfo.NumInstructions;
		
		// Build root parameters
		
		if (Input.IsRayTracingShader())
		{
			if (Input.Target.Frequency != SF_RayGen)
			{
				AppendRayTracingSystemParams(ShaderInfo.LocalRootParams);
			}
		}
		
		// Append permutation CRC to the entrypoint name to guarantee its uniqueness (and avoid potential collision when we create metallibs later).
		if (Input.IsRayTracingShader())
		{
			if (!ShaderInfo.RayEntryPoint.IsEmpty())
			{
				ShaderInfo.UniqueEntryPointName = ShaderInfo.RayEntryPoint;
				ShaderInfo.UniqueEntryPointName += FString::Printf(TEXT("_%0.8x_%0.8x"), ShaderInfo.CRCLen, ShaderInfo.CRC);
			}
			
			if (!ShaderInfo.RayAnyHitEntryPoint.IsEmpty())
			{
				ShaderInfo.UniqueRayAnyHitEntryPoint = ShaderInfo.RayAnyHitEntryPoint;
				ShaderInfo.UniqueRayAnyHitEntryPoint += FString::Printf(TEXT("_%0.8x_%0.8x"), ShaderInfo.CRCLen, ShaderInfo.CRC);
			}
			
			if (!ShaderInfo.RayIntersectionEntryPoint.IsEmpty())
			{
				ShaderInfo.UniqueRayIntersectionEntryPoint = ShaderInfo.RayIntersectionEntryPoint;
				ShaderInfo.UniqueRayIntersectionEntryPoint += FString::Printf(TEXT("_%0.8x_%0.8x"), ShaderInfo.CRCLen, ShaderInfo.CRC);
			}
		}
		else
		{
			ShaderInfo.UniqueEntryPointName = MainCRC;
		}
		
		// If we are using Shader Binding Layouts, then add to the root signature when linking
		TArray<IRRootParameter1> LinkingRootParams;
		
		if (Input.IsRayTracingShader())
		{
			for(uint32_t RegisterIdx = 0; RegisterIdx < FRHIShaderBindingLayout::MaxUniformBufferEntries; RegisterIdx++)
			{
				IRRootParameter1 RootParam;
				RootParam.ParameterType = IRRootParameterTypeCBV;
				RootParam.ShaderVisibility = IRShaderVisibility::IRShaderVisibilityAll;
				RootParam.Descriptor.ShaderRegister = RegisterIdx;
				RootParam.Descriptor.RegisterSpace = UE_HLSL_SPACE_STATIC_SHADER_BINDINGS;
				RootParam.Descriptor.Flags = IRRootDescriptorFlagDataStaticWhileSetAtExecute;
				
				LinkingRootParams.Add(RootParam);
			}
		}
		
		LinkingRootParams.Append(ShaderInfo.GlobalRootParams);
		
		// Create the root signature for air generation.
		IRVersionedRootSignatureDescriptor RootSignatureDesc;
		RootSignatureDesc.version = IRRootSignatureVersion_1_1;
		RootSignatureDesc.desc_1_1.Flags = IRRootSignatureFlagNone;
		RootSignatureDesc.desc_1_1.pStaticSamplers = StaticSamplerDescs;
		RootSignatureDesc.desc_1_1.NumStaticSamplers = UE_ARRAY_COUNT(StaticSamplerDescs);
		RootSignatureDesc.desc_1_1.pParameters = LinkingRootParams.GetData();
		RootSignatureDesc.desc_1_1.NumParameters = LinkingRootParams.Num();
		
		IRError* RootSignatureCreationError = nullptr;
		IRRootSignature* RootSignature = IRRootSignatureCreateFromDescriptor(&RootSignatureDesc, &RootSignatureCreationError);
		if (RootSignature == nullptr || RootSignatureCreationError != nullptr)
		{
			Output.AddErrorf(TEXT("Error: MetalShaderConverter failed to create a root signature for '%s' (%s)!"), *Input.EntryPointName, ANSI_TO_TCHAR((const char*)IRErrorGetPayload(RootSignatureCreationError)));
			Output.bSucceeded = false;
			
			return;
		}
		
		IRRootSignature* LocalRootSignature = nullptr;
		if (Input.IsRayTracingShader())
		{
			Algo::Sort(ShaderInfo.UnsortedLocalRootParams, [](const IRRootParameter1& LHS, const IRRootParameter1& RHS) 
			{
				check(LHS.ParameterType == IRRootParameterTypeCBV && RHS.ParameterType == IRRootParameterTypeCBV);
				return LHS.Descriptor.ShaderRegister < RHS.Descriptor.ShaderRegister;
			});
			
			for (IRRootParameter1& RootParam : ShaderInfo.UnsortedLocalRootParams)
			{
				ShaderInfo.LocalRootParams.Add(RootParam);
			}
			
			// Create the root signature for air generation.
			IRVersionedRootSignatureDescriptor LocalRootSignatureDesc;
			LocalRootSignatureDesc.version = IRRootSignatureVersion_1_1;
			LocalRootSignatureDesc.desc_1_1.Flags = IRRootSignatureFlagLocalRootSignature;
			LocalRootSignatureDesc.desc_1_1.pStaticSamplers = StaticSamplerDescs;
			LocalRootSignatureDesc.desc_1_1.NumStaticSamplers = UE_ARRAY_COUNT(StaticSamplerDescs);
			LocalRootSignatureDesc.desc_1_1.pParameters = ShaderInfo.LocalRootParams.GetData();
			LocalRootSignatureDesc.desc_1_1.NumParameters = ShaderInfo.LocalRootParams.Num();
			
			IRError* LocalRootSignatureCreationError = nullptr;
			LocalRootSignature = IRRootSignatureCreateFromDescriptor(&LocalRootSignatureDesc, &LocalRootSignatureCreationError);
			if (LocalRootSignature == nullptr || LocalRootSignatureCreationError != nullptr)
			{
				FShaderCompilerError Error(FString::Printf(TEXT("Error: MetalShaderConverter failed to create a local root signature for '%s' (error code: %s)!"), *Input.EntryPointName, ANSI_TO_TCHAR((const char *)IRErrorGetPayload(LocalRootSignatureCreationError))));
				Output.Errors.Add(Error);
				Output.bSucceeded = false;
				
				return;
			}
		}
		
		// Convert DXIL to air
		IRObject* DXILBytecode = IRObjectCreateFromDXIL(reinterpret_cast<const uint8_t*>(DxilData.GetData()), DxilData.Num() * sizeof(uint32), IRBytecodeOwnershipCopy);
		
		IRCompiler* CompilerInstance = IRCompilerCreate();
		IRCompilerSetGlobalRootSignature(CompilerInstance, RootSignature);
		IRCompilerSetStageInGenerationMode(CompilerInstance, IRStageInCodeGenerationModeUseSeparateStageInFunction);
		IRCompilerSetCompatibilityFlags(CompilerInstance, (IRCompatibilityFlags)(IRCompatibilityFlagBoundsCheck | IRCompatibilityFlagPositionInvariance | IRCompatibilityFlagSampleNanToZero | IRCompatibilityFlagTexWriteRoundingRTZ));
		IRCompilerSetMinimumGPUFamily(CompilerInstance, IRGPUFamilyMetal3);
		IRCompilerSetMinimumDeploymentTarget(CompilerInstance, IROperatingSystem_macOS, "15.0.0");
		
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
		IRCompilerEnableGeometryAndTessellationEmulation(CompilerInstance, Input.Environment.CompilerFlags.Contains(CFLAG_VertexToGeometryShader));
#endif
		
		if (LocalRootSignature != nullptr)
		{
			IRCompilerSetLocalRootSignature(CompilerInstance, LocalRootSignature);
			IRCompilerSetValidationFlags(CompilerInstance, IRCompilerValidationFlagNone);
		}
		
		IRRayTracingPipelineConfiguration* PipelineConfig = nullptr;
		
		if (Input.IsRayTracingShader())
		{	
			PipelineConfig = IRRayTracingPipelineConfigurationCreate();
			
			IRRayTracingPipelineConfigurationSetPipelineFlags(PipelineConfig, IRRaytracingPipelineFlagNone);
			IRRayTracingPipelineConfigurationSetIntrinsicMasks(PipelineConfig, IRIntrinsicMaskClosestHitAll, IRIntrinsicMaskMissShaderAll, IRIntrinsicMaskAnyHitShaderAll, IRIntrinsicMaskCallableShaderAll);
			IRRayTracingPipelineConfigurationSetMaxRecursiveDepth(PipelineConfig, RAY_TRACING_MAX_ALLOWED_RECURSION_DEPTH);
			IRRayTracingPipelineConfigurationSetRayGenerationCompilationMode(PipelineConfig, IRRayGenerationCompilationVisibleFunction);
			IRRayTracingPipelineConfigurationSetIntersectionFunctionCompilationMode(PipelineConfig, IRIntersectionFunctionCompilationVisibleFunction);
			IRRayTracingPipelineConfigurationSetMaxAttributeSizeInBytes(PipelineConfig, RAY_TRACING_MAX_ALLOWED_ATTRIBUTE_SIZE);
			
			IRCompilerSetRayTracingPipelineConfiguration(CompilerInstance, PipelineConfig);
			
			IRCompilerSetHitgroupType(CompilerInstance, ShaderInfo.bIsProcedural ? IRHitGroupTypeProceduralPrimitive : IRHitGroupTypeTriangles);
		}
		
		if (ShaderInfo.bUsesDualSourceBlending)
		{
			IRCompilerSetDualSourceBlendingConfiguration(CompilerInstance, IRDualSourceBlendingConfigurationForceEnabled);
		}

		CompileToMetalLib(ShaderInfo.MetalBytecode, ShaderInfo.DefaultBytecodeShaderStage, Input, Output, ReflectionJSON,
						  bDumpDebugInfo, bUsesDiscard, OutputSizeVS, MaxInputPrimitivesPerMeshThreadgroupGS, CCHeaderWriter, [&](IRError **CompileError) 
		{
			IRCompilerSetEntryPointName(CompilerInstance, TCHAR_TO_ANSI(*ShaderInfo.UniqueEntryPointName));
			return IRCompilerAllocCompileAndLink(CompilerInstance, (Input.IsRayTracingShader() && !ShaderInfo.RayEntryPoint.IsEmpty()) ? TCHAR_TO_ANSI(*ShaderInfo.RayEntryPoint) : TCHAR_TO_ANSI(*Input.EntryPointName), DXILBytecode, CompileError);
		});
		
		if (!Output.bSucceeded)
		{
			if(PipelineConfig)
			{
				IRRayTracingPipelineConfigurationDestroy(PipelineConfig);
			}
			IRObjectDestroy(DXILBytecode);
			IRCompilerDestroy(CompilerInstance);
			return;
		}
		
		const bool bHasExtendedEntryPoint = (!ShaderInfo.RayAnyHitEntryPoint.IsEmpty() || !ShaderInfo.RayIntersectionEntryPoint.IsEmpty());
		if (bHasExtendedEntryPoint)
		{
			check(Frequency == EShaderFrequency::SF_RayHitGroup);

			FString FinalExtendedEntryPointName;
			CompileToMetalLib(ShaderInfo.MetalBytecodeExtended, ShaderInfo.ExtendedBytecodeShaderStage, Input, Output, ReflectionJSON,
							  bDumpDebugInfo, bUsesDiscard, OutputSizeVS, MaxInputPrimitivesPerMeshThreadgroupGS, CCHeaderWriter, [&](IRError **CompileError) 
			{
				if (!ShaderInfo.RayAnyHitEntryPoint.IsEmpty() && !ShaderInfo.RayIntersectionEntryPoint.IsEmpty())
				{
					check(ShaderInfo.bIsProcedural);
					IRCompilerSetEntryPointName(CompilerInstance, TCHAR_TO_ANSI(*ShaderInfo.UniqueRayAnyHitEntryPoint));
					return IRCompilerAllocCombineCompileAndLink(CompilerInstance,
																TCHAR_TO_ANSI(*ShaderInfo.RayIntersectionEntryPoint), DXILBytecode, 
																TCHAR_TO_ANSI(*ShaderInfo.RayAnyHitEntryPoint), DXILBytecode,
																CompileError);
				}
				else if (!ShaderInfo.RayAnyHitEntryPoint.IsEmpty())
				{
					IRCompilerSetEntryPointName(CompilerInstance, TCHAR_TO_ANSI(*ShaderInfo.UniqueRayAnyHitEntryPoint));
					return IRCompilerAllocCompileAndLink(CompilerInstance, TCHAR_TO_ANSI(*ShaderInfo.RayAnyHitEntryPoint), DXILBytecode, CompileError);
				}
				else
				{
					IRCompilerSetEntryPointName(CompilerInstance, TCHAR_TO_ANSI(*ShaderInfo.UniqueRayIntersectionEntryPoint));
					return IRCompilerAllocCompileAndLink(CompilerInstance, TCHAR_TO_ANSI(*ShaderInfo.RayIntersectionEntryPoint), DXILBytecode, CompileError);
				}
			}, &FinalExtendedEntryPointName);

			if (!Output.bSucceeded)
			{
				if(PipelineConfig)
				{
					IRRayTracingPipelineConfigurationDestroy(PipelineConfig);
				}
				
				IRRootSignatureDestroy(RootSignature);
				IRObjectDestroy(DXILBytecode);
				IRCompilerDestroy(CompilerInstance);
				return;
			}

			// Bind the header to the symbol MSC actually emitted (may carry an LLVM ".N" suffix).
			if (!FinalExtendedEntryPointName.IsEmpty())
			{
				if (!ShaderInfo.RayAnyHitEntryPoint.IsEmpty())
				{
					ShaderInfo.UniqueRayAnyHitEntryPoint = FinalExtendedEntryPointName;
				}
				else
				{
					ShaderInfo.UniqueRayIntersectionEntryPoint = FinalExtendedEntryPointName;
				}
			}
		}
		Output.bSucceeded = true;
	
		if(PipelineConfig)
		{
			IRRayTracingPipelineConfigurationDestroy(PipelineConfig);
		}
		
		IRRootSignatureDestroy(LocalRootSignature);
		IRRootSignatureDestroy(RootSignature);
		
		IRObjectDestroy(DXILBytecode);
		IRCompilerDestroy(CompilerInstance);
		
		CCHeaderWriter.WriteSourceInfo(*Input.VirtualSourceFilePath, *Input.EntryPointName);
		CCHeaderWriter.WriteCompilerInfo();
		
		FString MetaData = CCHeaderWriter.ToString();
		MetaData += RTString;
		MetaData += TEXT("\n\n");
		if (ALNString.Len())
		{
			MetaData += TEXT("// Attributes: ");
			MetaData += ALNString;
			MetaData += TEXT("\n\n");
		}
		
		MetalSource = TCHAR_TO_UTF8(*MetaData);
		
		if (bDumpDebugInfo)
		{
			DumpDebugShaderBinary(Input, ShaderInfo.MetalBytecode.ObjectFile.GetData(), ShaderInfo.MetalBytecode.ObjectFile.Num() * sizeof(uint8), TEXT("air"));
		}
	}
#endif

	// Attribute [[clang::optnone]] causes performance hit with WPO on M1 Macs => replace with empty space
	const std::string ClangOptNoneString = "[[clang::optnone]]";
	for (size_t Begin = 0, End = 0; (Begin = MetalSource.find(ClangOptNoneString, End)) != std::string::npos; End = Begin)
	{
		MetalSource.replace(Begin, ClangOptNoneString.length(), " ");  
	}
	
	if (bDumpDebugInfo && !MetalSource.empty())
	{
		DumpDebugShaderText(Input, &MetalSource[0], MetalSource.size(), TEXT("metal"));
	}

	Output.Target = Input.Target;
	
	BuildMetalShaderOutput(Output,
		Input,
		ShaderParameterParser,
		MetalSource.c_str(),
		MetalSource.length(),
		ShaderInfo.CRCLen,
		ShaderInfo.CRC,
		VersionEnum,
		*Standard,
		*MinOSVersion,
		Output.Errors,
		OutputData.TypedBuffers,
		OutputData.InvariantBuffers,
		OutputData.TypedUAVs,
		OutputData.ConstantBuffers,
		bAllowFastIntrinsics,
		ShaderInfo.NumCBVs,
		OutputSizeVS,
		MaxInputPrimitivesPerMeshThreadgroupGS,
		bUsesDiscard,
		ReflectionJSON,
		{},
		&ShaderInfo);
}

#endif // PLATFORM_MAC || PLATFORM_WINDOWS
