// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ShaderCompilerCommon.h"
#include "MetalShaderCompiler.h"
#include "HlslccHeaderWriter.h"
#include "MetalShaderFormat.h"
#include "MetalShaderResources.h"

class FMetalCompileShaderMSC
{
public:
	static void DoCompileMetalShader(
		const FShaderCompilerInput& Input,
		const FShaderParameterParser& ShaderParameterParser,
		FShaderCompilerOutput& Output,
		const FString& InPreprocessedShader,
		uint32 VersionEnum,
		EMetalGPUSemantics Semantics,
		uint32 MaxUnrollLoops,
		EShaderFrequency Frequency,
		bool bDumpDebugInfo,
		const FString& Standard,
		const FString& MinOSVersion,
		FMetalBindlessShaderInfo& ShaderInfo
	);
};

