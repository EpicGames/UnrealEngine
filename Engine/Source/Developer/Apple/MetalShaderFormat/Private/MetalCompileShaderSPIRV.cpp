// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetalCompileShaderSPIRV.h"
#include "MetalShaderCompiler.h"

#include "MetalShaderFormat.h"
#include "MetalShaderResources.h"
#include "MetalBackend.h"
#include "RHIDefinitions.h"
#include "ShaderCompilerDefinitions.h"
#include "SpirvReflectCommon.h"
#include "ShaderParameterParser.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Containers/AnsiString.h"

#include <regex>

extern void BuildMetalShaderOutput(
	FShaderCompilerOutput& ShaderOutput,
	const FShaderCompilerInput& ShaderInput,
	const FShaderParameterParser& ShaderParameterParser,
	const ANSICHAR* InShaderSource,
	uint32 SourceLen,
	uint32 SourceCRCLen,
	uint32 SourceCRC,
	uint32 Version,
	TCHAR const* Standard,
	TCHAR const* MinOSVersion,
	TArray<FShaderCompilerError>& OutErrors,
	uint32 TypedBuffers,
	uint32 InvariantBuffers,
	uint32 TypedUAVs,
	uint32 ConstantBuffers,
	bool bAllowFastIntriniscs,
	uint32 NumCBVs,
	uint32 OutputSizeVS,
	uint32 MaxInputPrimitivesPerMeshThreadgroupGS,
	const bool bUsesDiscard,
	char const* ShaderReflectionJSON,
	FMetalShaderBytecode const& CompiledShaderBytecode,
	FMetalBindlessShaderInfo* ShaderInfo
);

static void Patch16bitInHlslSource(const FShaderCompilerInput& Input, FAnsiString& SourceData)
{	
	TArray<int32> PatchPositions;
	
	static const FAnsiStringView TextureStr = "Texture";
	static const FAnsiStringView HalfStr("half");
	static const FAnsiStringView FloatStr("float");
	
	// Find all half texture and buffer types to patch
	{
		int32 Pos = SourceData.Find(TextureStr );
		while (Pos != INDEX_NONE)
		{
			Pos += TextureStr.Len();
			if (Pos >= SourceData.Len())
			{
				break;
			}
			static const FAnsiStringView TextureTypeNames[] =
			{
				"1D<",
				"1DArray<",
				"2D<",
				"2DArray<",
				"3D<",
				"Cube<",
				"CubeArray<"
			};
			
			FAnsiStringView CodeStr(&SourceData[Pos], SourceData.Len() - Pos);
			for (const FAnsiStringView& TextureTypeStr : TextureTypeNames)
			{
				if (CodeStr.StartsWith(TextureTypeStr))
				{
					CodeStr.RightChopInline(TextureTypeStr.Len());
					CodeStr.TrimStartInline();
					if (CodeStr.StartsWith(HalfStr))
					{
						PatchPositions.Push(CodeStr.GetData() - *SourceData);
					}
				}
			}
			
			Pos = SourceData.Find(TextureStr, ESearchCase::CaseSensitive, ESearchDir::FromStart, Pos);
		}
	}
	
	{
		static const FAnsiStringView BufferStr = "Buffer<";
		int32 Pos = SourceData.Find(BufferStr);
		while (Pos != INDEX_NONE)
		{
			Pos += BufferStr.Len();
			if (Pos >= SourceData.Len())
			{
				break;
			}

			FAnsiStringView CodeStr(&SourceData[Pos], SourceData.Len() - Pos);
			CodeStr.TrimStartInline();
			if (CodeStr.StartsWith(HalfStr))
			{
				PatchPositions.Push(CodeStr.GetData() - *SourceData);
			}
			Pos = SourceData.Find(BufferStr, ESearchCase::CaseSensitive, ESearchDir::FromStart, Pos);
		}
	}
	
	// Convert global uniforms to float (uniforms in uniform buffers are always float via shader macro UB_HALF_FLOAT)
	FAnsiStringView CodeStr = SourceData;
	while(CodeStr.Len() > 0)
	{
		CodeStr.TrimStartInline();
		if (CodeStr.StartsWith(HalfStr))
		{
			int32 PatchPos = CodeStr.GetData() - *SourceData;
			CodeStr.RemovePrefix(HalfStr.Len());
			char Head = CodeStr[0];
			if (Head == '1' || Head == '2' || Head == '3' || Head == '4')
			{
				CodeStr.RemovePrefix(1);
				Head = CodeStr[0];
			}
			if (isspace(Head))
			{
				CodeStr.TrimStartInline();
				Head = CodeStr[0];
				if (isalpha(Head) || Head == '_')
				{
					for (char C : CodeStr)
					{
						if (isalnum(C) || isspace(C) || C == '_')
						{
							continue;
						}
						else if (C == ';')
						{
							PatchPositions.Push(PatchPos);
							break;
						}
						else
						{
							break;
						}
					}
				}
			}
		}
		// This is brittle, but only consider "half" types at the beginning of a new line of code.
		// We only want to replace global uniforms, not local variables or variables inside uniform buffers.
		int32 FindPos = CodeStr.Find("\nhalf", 1);
		if (FindPos == INDEX_NONE)
		{
			break;
		}
		CodeStr.RemovePrefix(FindPos + 1);
	}

	// Create new code string where all relevant "half" strings are replaced with "float"
	FAnsiString Result = FAnsiString::ConstructWithSlack("", SourceData.Len() + PatchPositions.Num());
	int32 LastEndPatchPos = 0;
	for (const int32 Pos : PatchPositions)
	{
		Result.Append(&SourceData[LastEndPatchPos], Pos - LastEndPatchPos);
		Result.Append(FloatStr);
		LastEndPatchPos = Pos + HalfStr.Len();
	}
	Result.Append(&SourceData[LastEndPatchPos], SourceData.Len() - LastEndPatchPos);
	SourceData = std::move(Result);
}

void FMetalCompileShaderSPIRV::DoCompileMetalShader(
	const FShaderCompilerInput& Input,
	const FShaderParameterParser& ShaderParameterParser,
	FShaderCompilerOutput& Output,
	const FString& InPreprocessedShader,
	uint32 VersionEnum,
	EMetalGPUSemantics Semantics,
	uint32 MaxUnrollLoops,
	EShaderFrequency Frequency,
	bool bDumpDebugInfo,
	const FString& Standard,
	const FString& MinOSVersion)
{
	int32 IABTier = VersionEnum >= 4 ? Input.Environment.GetCompileArgument(TEXT("METAL_INDIRECT_ARGUMENT_BUFFERS"), 0) : 0;

	Output.bSucceeded = false;

	std::string MetalSource;
	FString MetalErrors;
	
	bool const bZeroInitialise = Input.Environment.CompilerFlags.Contains(CFLAG_ZeroInitialise);
	bool const bBoundsChecks = Input.Environment.CompilerFlags.Contains(CFLAG_BoundsChecking);

	bool bAllowFastIntrinsics = Input.Environment.GetCompileArgument(TEXT("METAL_USE_FAST_INTRINSICS"), false);

	// WPO requires that we make all multiply/sincos instructions invariant :(
	bool bForceInvariance = Input.Environment.GetCompileArgument(TEXT("USES_WORLD_POSITION_OFFSET"), false);
	
	FMetalShaderOutputMetaData OutputData;
	
	uint32 CRCLen = 0;
	uint32 CRC = 0;
	uint32 SourceLen = 0;
	int32 Result = 0;
	
	struct FMetalResourceTableEntry : FUniformResourceEntry
	{
		FString Name;
		uint32 Size;
		uint32 SetIndex;
		bool bUsed;
	};
	TMap<FString, TArray<FMetalResourceTableEntry>> IABs;

	FString PreprocessedShader = InPreprocessedShader;
	bool bUsingInlineRayTracing = Input.Environment.CompilerFlags.Contains(CFLAG_InlineRayTracing);
	const bool bOutputAnalysisArtifacts = Input.Environment.CompilerFlags.Contains(CFLAG_OutputAnalysisArtifacts);

#if PLATFORM_MAC || PLATFORM_WINDOWS
	{
		std::string EntryPointNameAnsi(TCHAR_TO_UTF8(*Input.EntryPointName));

		CrossCompiler::FShaderConductorContext CompilerContext;

		// Initialize compilation options for ShaderConductor
		CrossCompiler::FShaderConductorOptions Options(Input.Environment.CompilerFlags);

		Options.TargetEnvironment = CrossCompiler::FShaderConductorOptions::ETargetEnvironment::Vulkan_1_1;

		// Always disable FMA pass for Pixel and Compute shader,
		// otherwise determine whether [[position, invariant]] qualifier is available in Metal or not.
		if (Frequency == SF_Pixel || Frequency == SF_Compute)
		{
			Options.bEnableFMAPass = false;
		}
		else
		{
			Options.bEnableFMAPass = bForceInvariance;
		}
		
		if(!Input.Environment.FullPrecisionInPS)
		{
			Options.bEnable16bitTypes = true;
		}
		
		// Load shader source into compiler context
		CompilerContext.LoadSource(PreprocessedShader, Input.VirtualSourceFilePath, Input.EntryPointName, Frequency);

		// Convert shader source to ANSI string
		FAnsiString SourceData = FAnsiString::ConstructFromPtrSize(CompilerContext.GetSourceString(), CompilerContext.GetSourceLength());

		// Replace special case texture "gl_LastFragData" by native subpass fetch operation
		static const uint32 MaxMetalSubpasses = 8;
		uint32 SubpassInputsDim[MaxMetalSubpasses];

		if (bDumpDebugInfo)
		{
			DumpDebugShaderText(Input, &SourceData[0], SourceData.Len(), TEXT("orig.hlsl"));
		}
		
		bool bSourceDataWasModified = PatchSpecialTextureInHlslSource(SourceData, SubpassInputsDim, MaxMetalSubpasses);
		
		// If using 16 bit types disable half precision in constant buffer due to errors in layout
		if(Options.bEnable16bitTypes)
		{
			Patch16bitInHlslSource(Input, SourceData);
			bSourceDataWasModified = true;
		}
		
		// If source data was modified, reload it into the compiler context
		if (bSourceDataWasModified)
		{
			CompilerContext.LoadSource(FAnsiStringView(*SourceData, SourceData.Len()), Input.VirtualSourceFilePath, Input.EntryPointName, Frequency);

			if (bOutputAnalysisArtifacts)
			{
				Output.AddStatistic<FString>(TEXT("Modified HLSL"), ANSI_TO_TCHAR(&SourceData[0]), FGenericShaderStat::EFlags::Hidden, FShaderStatTagNames::AnalysisArtifactsName);
			}
		}

		if (bDumpDebugInfo)
		{
			DumpDebugShaderText(Input, &SourceData[0], SourceData.Len(), TEXT("rewritten.hlsl"));
		}

		CrossCompiler::FHlslccHeaderWriter CCHeaderWriter;

		FString ALNString;
		FString RTString;
		uint32 IABOffsetIndex = 0;
		uint64 BufferIndices = 0xffffffffffffffff;

		// Compile HLSL source to SPIR-V binary
		TArray<uint32> SpirvData;
		if (CompilerContext.CompileHlslToSpirv(Options, SpirvData))
		{
			Result = 1;

			// Dump SPIRV module before code reflection so we can analyse the dumped output as early as possible (in case of issues in SPIRV-Reflect)
			if (bDumpDebugInfo)
			{
				DumpDebugShaderBinary(Input, SpirvData.GetData(), SpirvData.Num() * sizeof(uint32), TEXT("spv"));
				DumpDebugShaderDisassembledSpirv(Input, SpirvData.GetData(), SpirvData.Num() * sizeof(uint32), TEXT("spvasm"));
			}
			
			if (bOutputAnalysisArtifacts)
			{
				FGenericShaderStat ShaderIRReflection;
				if (CompilerContext.Disassemble(CrossCompiler::EShaderConductorIR::Spirv, SpirvData.GetData(), SpirvData.Num() * sizeof(uint32), ShaderIRReflection))
				{
					Output.ShaderStatistics.Add(MoveTemp(ShaderIRReflection));
				}
			}

			// Now perform reflection on the SPIRV and tweak any decorations that we need to.
			// This used to be done via JSON, but that was slow and alloc happy so use SPIRV-Reflect instead.
			spv_reflect::ShaderModule Reflection(SpirvData.Num() * sizeof(uint32), SpirvData.GetData());
			checkf(Reflection.GetResult() == SPV_REFLECT_RESULT_SUCCESS, TEXT("SPIRV-reflect failed with error %d"), Reflection.GetResult());
			
			SpvReflectResult SPVRResult = SPV_REFLECT_RESULT_NOT_READY;
			uint32 Count = 0;
			FSpirvReflectBindings ReflectionBindings;
			TArray<SpvReflectDescriptorBinding*> Bindings;
			TArray<SpvReflectBlockVariable*> ConstantBindings;
			TArray<SpvExecutionMode> ExecutionModes;
			
			uint32 UAVIndices = 0xffffffff;
			uint64 TextureIndices = 0xffffffffffffffff;
			uint64 SamplerIndices = 0xffffffffffffffff;
			
			TArray<FString> TableNames;
			TMap<FString, FMetalResourceTableEntry> ResourceTable;
			if (IABTier >= 1)
			{
				for (auto Pair : Input.Environment.UniformBufferMap)
				{
					TableNames.Add(*Pair.Key);
				}
				
				for (const FUniformResourceEntry& Entry : Input.Environment.ResourceTableMap.Resources)
				{
					TArray<FMetalResourceTableEntry>& Resources = IABs.FindOrAdd(FString(Entry.GetUniformBufferName()));
					if ((uint32)Resources.Num() <= Entry.ResourceIndex)
					{
						Resources.SetNum(Entry.ResourceIndex + 1);
					}
					FMetalResourceTableEntry NewEntry;
					NewEntry.UniformBufferMemberName = Entry.UniformBufferMemberName;
					NewEntry.UniformBufferNameLength = Entry.UniformBufferNameLength;
					NewEntry.Type = Entry.Type;
					NewEntry.ResourceIndex = Entry.ResourceIndex;
					NewEntry.Name = Entry.UniformBufferMemberName;
					NewEntry.Size = 1;
					NewEntry.bUsed = false;
					Resources[Entry.ResourceIndex] = NewEntry;
				}
				
				for (uint32 i = 0; i < (uint32)TableNames.Num(); )
				{
					if (!IABs.Contains(TableNames[i]))
					{
						TableNames.RemoveAt(i);
					}
					else
					{
						i++;
					}
				}
				
				for (auto Pair : IABs)
				{
					uint32 Index = 0;
					for (uint32 i = 0; i < (uint32)Pair.Value.Num(); i++)
					{
						FMetalResourceTableEntry& Entry = Pair.Value[i];
						switch(Entry.Type)
						{
							case UBMT_UAV:
							case UBMT_RDG_TEXTURE_UAV:
							case UBMT_RDG_BUFFER_UAV:
								Entry.ResourceIndex = Index;
								Entry.Size = 1;
								Index += 2;
								break;
							default:
								Entry.ResourceIndex = Index;
								Index++;
								break;
						}
						for (uint32 j = 0; j < (uint32)TableNames.Num(); j++)
						{
							if (Entry.GetUniformBufferName() == TableNames[j])
							{
								Entry.SetIndex = j;
								break;
							}
						}
						ResourceTable.Add(Entry.Name, Entry);
					}
				}
			}
			
			{
				if (Frequency == SF_Compute)
				{
					const SpvReflectEntryPoint* EntryPoint = spvReflectGetEntryPoint(&Reflection.GetShaderModule(), Reflection.GetEntryPointName());
					if (EntryPoint)
					{
						check(EntryPoint->shader_stage == SPV_REFLECT_SHADER_STAGE_COMPUTE_BIT);
						CCHeaderWriter.WriteNumThreads(EntryPoint->local_size.x, EntryPoint->local_size.y, EntryPoint->local_size.z);
					}
				}
			}
			
			Count = 0;
			SPVRResult = Reflection.EnumerateDescriptorBindings(&Count, nullptr);
			check(SPVRResult == SPV_REFLECT_RESULT_SUCCESS);
			Bindings.SetNum(Count);
			SPVRResult = Reflection.EnumerateDescriptorBindings(&Count, Bindings.GetData());
			check(SPVRResult == SPV_REFLECT_RESULT_SUCCESS);
			if (Count > 0)
			{
				TArray<SpvReflectDescriptorBinding*> ResourceBindings;
				TArray<SpvReflectDescriptorBinding*> ArgumentBindings;
				TSet<FString> UsedSets;
				
				// Extract all the bindings first so that we process them in order - this lets us assign UAVs before other resources
				// Which is necessary to match the D3D binding scheme.
				for (SpvReflectDescriptorBinding* Binding : Bindings)
				{
					const FString BindingName(Binding->name);

					if (Binding->resource_type != SPV_REFLECT_RESOURCE_FLAG_CBV && ResourceTable.Contains(BindingName))
					{
						ResourceBindings.Add(Binding);
						
						FMetalResourceTableEntry Entry = ResourceTable.FindRef(BindingName);
						UsedSets.Add(FString(Entry.GetUniformBufferName()));
						
						continue;
					}

					// Add descriptor binding to argument bindings if it's a constant buffer with a name from 'TableNames'. Otherwise, add to common binding container.
					if (Binding->resource_type == SPV_REFLECT_RESOURCE_FLAG_CBV && Binding->accessed && TableNames.Contains(BindingName))
					{
						check(Binding->descriptor_type == SPV_REFLECT_DESCRIPTOR_TYPE_UNIFORM_BUFFER);
						ArgumentBindings.Add(Binding);
					}
					else
					{
						ReflectionBindings.AddDescriptorBinding(Binding);
					}
				}
				
				for (uint32 i = 0; i < (uint32)TableNames.Num(); )
				{
					if (UsedSets.Contains(TableNames[i]))
					{
						IABs.FindChecked(TableNames[i])[0].SetIndex = i;
						i++;
					}
					else
					{
						IABs.Remove(TableNames[i]);
						TableNames.RemoveAt(i);
					}
				}
				
				for (uint32 i = 0; i < (uint32)ArgumentBindings.Num(); )
				{
					FString Name = UTF8_TO_TCHAR(ArgumentBindings[i]->name);
					if (TableNames.Contains(Name))
					{
						auto* ResourceArray = IABs.Find(Name);
						auto const& LastResource = ResourceArray->Last();
						uint32 ResIndex = LastResource.ResourceIndex + LastResource.Size;
						uint32 SetIndex = SPV_REFLECT_SET_NUMBER_DONT_CHANGE;
						for (uint32 j = 0; j < (uint32)TableNames.Num(); j++)
						{
							if (Name == TableNames[j])
							{
								SetIndex = j;
								break;
							}
						}
						
						FMetalResourceTableEntry Entry;
						Entry.UniformBufferMemberName = LastResource.UniformBufferMemberName;
						Entry.UniformBufferNameLength = LastResource.UniformBufferNameLength;
						Entry.Name = Name;
						Entry.ResourceIndex = ResIndex;
						Entry.SetIndex = SetIndex;
						Entry.bUsed = true;
						
						ResourceArray->Add(Entry);
						ResourceTable.Add(Name, Entry);
						
						ResourceBindings.Add(ArgumentBindings[i]);
						
						i++;
					}
					else
					{
						ReflectionBindings.UniformBuffers.Add(ArgumentBindings[i]);
						ArgumentBindings.RemoveAt(i);
					}
				}
				
				const uint32 GlobalSetId = 32;
				
				for (auto const& Binding : ReflectionBindings.TBufferUAVs)
				{
					check(UAVIndices);
					uint32 Index = FPlatformMath::CountTrailingZeros(UAVIndices);
					
					// UAVs always claim all slots so we don't have conflicts as D3D expects 0-7
					BufferIndices &= ~(1ull << (uint64)Index);
					TextureIndices &= ~(1ull << (uint64)Index);
					UAVIndices &= ~(1 << Index);
					
					OutputData.TypedUAVs |= (1 << Index);
					OutputData.TypedBuffers |= (1 << Index);
					
					CCHeaderWriter.WriteUAV(UTF8_TO_TCHAR(Binding->name), Index);
					
					SPVRResult = Reflection.ChangeDescriptorBindingNumbers(Binding, Index, GlobalSetId);
					check(SPVRResult == SPV_REFLECT_RESULT_SUCCESS);
				}
				
				for (auto const& Binding : ReflectionBindings.SBufferUAVs)
				{
					check(UAVIndices);
					uint32 Index = FPlatformMath::CountTrailingZeros(UAVIndices);
					
					// UAVs always claim all slots so we don't have conflicts as D3D expects 0-7
					BufferIndices &= ~(1ull << (uint64)Index);
					TextureIndices &= ~(1ull << (uint64)Index);
					UAVIndices &= ~(1 << Index);
					
					OutputData.InvariantBuffers |= (1 << Index);
					
					CCHeaderWriter.WriteUAV(UTF8_TO_TCHAR(Binding->name), Index);

					SPVRResult = Reflection.ChangeDescriptorBindingNumbers(Binding, Index, GlobalSetId);
					check(SPVRResult == SPV_REFLECT_RESULT_SUCCESS);
				}
				
				for (auto const& Binding : ReflectionBindings.TextureUAVs)
				{
					check(UAVIndices);
					uint32 Index = FPlatformMath::CountTrailingZeros(UAVIndices);
					
					// UAVs always claim all slots so we don't have conflicts as D3D expects 0-7
					// For texture2d this allows us to emulate atomics with buffers
					BufferIndices &= ~(1ull << (uint64)Index);
					TextureIndices &= ~(1ull << (uint64)Index);
					UAVIndices &= ~(1 << Index);
					
					CCHeaderWriter.WriteUAV(UTF8_TO_TCHAR(Binding->name), Index);

					SPVRResult = Reflection.ChangeDescriptorBindingNumbers(Binding, Index, GlobalSetId);
					check(SPVRResult == SPV_REFLECT_RESULT_SUCCESS);
				}
				
				IABOffsetIndex = FPlatformMath::CountTrailingZeros64(BufferIndices);
				
				TMap<FString, uint32> IABTier1Index;
				if (IABTier == 1)
				{
					for (auto const& Binding : ResourceBindings)
					{
						FMetalResourceTableEntry* Entry = ResourceTable.Find(FString(Binding->name));
						FString EntryUniformBufferName(Entry->GetUniformBufferName());
						auto* ResourceArray = IABs.Find(EntryUniformBufferName);
						if (!IABTier1Index.Contains(EntryUniformBufferName))
						{
							IABTier1Index.Add(EntryUniformBufferName, 0);
						}
						if (Binding->descriptor_type == SPV_REFLECT_DESCRIPTOR_TYPE_STORAGE_BUFFER)
						{
							bool bFoundBufferSizes = false;
							for (auto& Resource : *ResourceArray)
							{
								if (Resource.ResourceIndex == 65535)
								{
									bFoundBufferSizes = true;
									break;
								}
							}
							if (!bFoundBufferSizes)
							{
								FMetalResourceTableEntry BufferSizes;
								BufferSizes.UniformBufferMemberName = Entry->UniformBufferMemberName;
								BufferSizes.UniformBufferNameLength = Entry->UniformBufferNameLength;
								BufferSizes.Name = TEXT("BufferSizes");
								BufferSizes.Type = UBMT_SRV;
								BufferSizes.ResourceIndex = 65535;
								BufferSizes.SetIndex = Entry->SetIndex;
								BufferSizes.Size = 1;
								BufferSizes.bUsed = true;
								ResourceArray->Insert(BufferSizes, 0);
								IABTier1Index[EntryUniformBufferName] = 1;
							}
						}
					}
				}
				
				for (auto const& Binding : ResourceBindings)
				{
					FMetalResourceTableEntry* Entry = ResourceTable.Find(FString(Binding->name));
					FString EntryUniformBufferName(Entry->GetUniformBufferName());
					
					for (uint32 j = 0; j < (uint32)TableNames.Num(); j++)
					{
						if (EntryUniformBufferName == TableNames[j])
						{
							Entry->SetIndex = j;
							BufferIndices &= ~(1ull << ((uint64)j + IABOffsetIndex));
							TextureIndices &= ~(1ull << ((uint64)j + IABOffsetIndex));
							break;
						}
					}
					Entry->bUsed = true;
					
					auto* ResourceArray = IABs.Find(EntryUniformBufferName);
					uint32 ResourceIndex = Entry->ResourceIndex;
					if (IABTier == 1)
					{
						for (auto& Resource : *ResourceArray)
						{
							Resource.SetIndex = Entry->SetIndex;
							if (Resource.ResourceIndex == Entry->ResourceIndex)
							{
								uint32& Tier1Index = IABTier1Index.FindChecked(EntryUniformBufferName);
								ResourceIndex = Tier1Index++;
								Resource.bUsed = true;
								break;
							}
						}
						if (Entry->ResourceIndex != 65535)
						{
							SPVRResult = Reflection.ChangeDescriptorBindingNumbers(Binding, ResourceIndex, Entry->SetIndex);
							check(SPVRResult == SPV_REFLECT_RESULT_SUCCESS);
						}
					}
					else
					{
						for (auto& Resource : *ResourceArray)
						{
							if (Resource.Name == Entry->Name)
							{
								Resource.SetIndex = Entry->SetIndex;
								Resource.bUsed = true;
								break;
							}
						}
						SPVRResult = Reflection.ChangeDescriptorBindingNumbers(Binding, Entry->ResourceIndex + 1, Entry->SetIndex);
						check(SPVRResult == SPV_REFLECT_RESULT_SUCCESS);
					}
				}
				
				for (auto const& Pair : IABs)
				{
					const FString& Name = Pair.Key;
					auto const& ResourceArray = Pair.Value;
					uint32 SetIndex = ResourceArray[0].SetIndex + IABOffsetIndex;

					TArray<uint32> IndirectArgumentBufferIndices;
					IndirectArgumentBufferIndices.Reserve(ResourceArray.Num());
					for (auto const& Resource : ResourceArray)
					{
						if (Resource.bUsed)
						{
							IndirectArgumentBufferIndices.Add((Resource.ResourceIndex == 65535 ? 0 : Resource.ResourceIndex + 1));
						}
					}

					CCHeaderWriter.WriteArgumentBuffers(SetIndex, IndirectArgumentBufferIndices);
					CCHeaderWriter.WriteUniformBlock(*Name, SetIndex);
				}
				
				for (auto const& Binding : ReflectionBindings.UniformBuffers)
				{
					check(BufferIndices);
					uint32 Index = FPlatformMath::CountTrailingZeros64(BufferIndices);
					BufferIndices &= ~(1ull << (uint64)Index);
					
					OutputData.ConstantBuffers |= (1 << Index);
					
					// Global uniform buffer - handled specially as we care about the internal layout
					if (strstr(Binding->name, "$Globals"))
					{
						TCBDMARangeMap CBRanges;
						CCHeaderWriter.WritePackedUB(Index);
						
						FString MbrString;
						for (uint32 i = 0; i < Binding->block.member_count; i++)
						{
							SpvReflectBlockVariable& member = Binding->block.members[i];
							
							CCHeaderWriter.WritePackedUBField(UTF8_TO_TCHAR(member.name), member.absolute_offset, member.size);
							
							const uint32 MbrOffset = member.absolute_offset / sizeof(float);
							const uint32 MbrSize = member.size / sizeof(float);
							unsigned DestCBPrecision = TEXT('h');
							unsigned SourceOffset = MbrOffset;
							unsigned DestOffset = MbrOffset;
							unsigned DestSize = MbrSize;
							unsigned DestCBIndex = 0;
							InsertRange(CBRanges, Index, SourceOffset, DestSize, DestCBIndex, DestCBPrecision, DestOffset);
						}
						
						for (auto Iter = CBRanges.begin(); Iter != CBRanges.end(); ++Iter)
						{
							TDMARangeList& List = Iter->second;
							for (auto IterList = List.begin(); IterList != List.end(); ++IterList)
							{
								check(IterList->DestCBIndex == 0);
								CCHeaderWriter.WritePackedUBGlobalCopy(IterList->SourceCB, IterList->SourceOffset, IterList->DestCBIndex, IterList->DestCBPrecision, IterList->DestOffset, IterList->Size);
							}
						}
					}
					else
					{
						// Regular uniform buffer - we only care about the binding index
						CCHeaderWriter.WriteUniformBlock(UTF8_TO_TCHAR(Binding->name), Index);
					}
					
					SPVRResult = Reflection.ChangeDescriptorBindingNumbers(Binding, Index, GlobalSetId);
					check(SPVRResult == SPV_REFLECT_RESULT_SUCCESS);
				}
				
				for (auto const& Binding : ReflectionBindings.SBufferSRVs)
				{
					check(BufferIndices);
					uint32 Index = FPlatformMath::CountTrailingZeros64(BufferIndices);
					
					BufferIndices &= ~(1ull << (uint64)Index);
					
					OutputData.InvariantBuffers |= (1 << Index);
					
					CCHeaderWriter.WriteSRV(UTF8_TO_TCHAR(Binding->name), Index);

					SPVRResult = Reflection.ChangeDescriptorBindingNumbers(Binding, Index, GlobalSetId);
					check(SPVRResult == SPV_REFLECT_RESULT_SUCCESS);
				}
				
				for (auto const& Binding : ReflectionBindings.AccelerationStructures)
				{
					check(BufferIndices);
					uint32 Index = FPlatformMath::CountTrailingZeros64(BufferIndices);

					BufferIndices &= ~(1ull << (uint64)Index);

					OutputData.InvariantBuffers |= (1 << Index);

					CCHeaderWriter.WriteSRV(UTF8_TO_TCHAR(Binding->name), Index);

					SPVRResult = Reflection.ChangeDescriptorBindingNumbers(Binding, Index, GlobalSetId);
					check(SPVRResult == SPV_REFLECT_RESULT_SUCCESS);
				}

				
				
				for (auto const& Binding : ReflectionBindings.TBufferSRVs)
				{
					check(TextureIndices);
					uint32 Index = FPlatformMath::CountTrailingZeros64(TextureIndices);
					TextureIndices &= ~(1ull << uint64(Index));
					
					OutputData.TypedBuffers |= (1 << Index);
					
					CCHeaderWriter.WriteSRV(UTF8_TO_TCHAR(Binding->name), Index);
					
					SPVRResult = Reflection.ChangeDescriptorBindingNumbers(Binding, Index, GlobalSetId);
					check(SPVRResult == SPV_REFLECT_RESULT_SUCCESS);
				}
				
				for (auto const& Binding : ReflectionBindings.TextureSRVs)
				{
					check(TextureIndices);
					uint32 Index = FPlatformMath::CountTrailingZeros64(TextureIndices);
					TextureIndices &= ~(1ull << uint64(Index));
					
					CCHeaderWriter.WriteSRV(UTF8_TO_TCHAR(Binding->name), Index);
					
					SPVRResult = Reflection.ChangeDescriptorBindingNumbers(Binding, Index, GlobalSetId);
					check(SPVRResult == SPV_REFLECT_RESULT_SUCCESS);
				}
				
				for (auto const& Binding : ReflectionBindings.Samplers)
				{
					check(SamplerIndices);
					uint32 Index = FPlatformMath::CountTrailingZeros64(SamplerIndices);
					SamplerIndices &= ~(1ull << (uint64)Index);
					
					CCHeaderWriter.WriteSamplerState(UTF8_TO_TCHAR(Binding->name), Index);
					
					SPVRResult = Reflection.ChangeDescriptorBindingNumbers(Binding, Index, GlobalSetId);
					check(SPVRResult == SPV_REFLECT_RESULT_SUCCESS);
				}
			}
			
			if (Frequency == SF_Pixel)
			{
				ReflectionBindings.GatherOutputAttributes(Reflection);
				for (auto const& Var : ReflectionBindings.OutputAttributes)
				{
					if (Var->storage_class == SpvStorageClassOutput && Var->built_in == -1 && strstr(Var->name, "SV_Target"))
					{
						FString TypeQualifier;
							
						auto const type = *Var->type_description;
						uint32_t masked_type = type.type_flags & 0xF;
							
						switch (masked_type) {
							default: checkf(false, TEXT("unsupported component type %d"), masked_type); break;
							case SPV_REFLECT_TYPE_FLAG_BOOL  : TypeQualifier = TEXT("b"); break;
							case SPV_REFLECT_TYPE_FLAG_INT   : TypeQualifier = (type.traits.numeric.scalar.signedness ? TEXT("i") : TEXT("u")); break;
							case SPV_REFLECT_TYPE_FLAG_FLOAT : TypeQualifier = (type.traits.numeric.scalar.width == 32 ? TEXT("f") : TEXT("h")); break;
						}
							
						if (type.type_flags & SPV_REFLECT_TYPE_FLAG_MATRIX)
						{
							TypeQualifier += FString::Printf(TEXT("%d%d"), type.traits.numeric.matrix.row_count, type.traits.numeric.matrix.column_count);
						}
						else if (type.type_flags & SPV_REFLECT_TYPE_FLAG_VECTOR)
						{
							TypeQualifier += FString::Printf(TEXT("%d"), type.traits.numeric.vector.component_count);
						}
						else
						{
							TypeQualifier += TEXT("1");
						}
							
						CCHeaderWriter.WriteOutputAttribute(TEXT("SV_Target"), *TypeQualifier, Var->location, /*bLocationPrefix:*/ false, /*bLocationSuffix:*/ true);
					}
				}
			}
			
			if (Frequency == SF_Vertex)
			{
				uint32 AssignedInputs = 0;

				ReflectionBindings.GatherInputAttributes(Reflection);
				for (auto const& Var : ReflectionBindings.InputAttributes)
				{
					if (Var->storage_class == SpvStorageClassInput && Var->built_in == -1)
					{
						unsigned Location = Var->location;
						unsigned SemanticIndex = Location;
						check(Var->semantic);
						unsigned i = (unsigned)strlen(Var->semantic);
						check(i);
						while (isdigit((unsigned char)(Var->semantic[i-1])))
						{
							i--;
						}
						if (i < strlen(Var->semantic))
						{
							SemanticIndex = (unsigned)atoi(Var->semantic + i);
							if (Location != SemanticIndex)
							{
								Location = SemanticIndex;
							}
						}
							
						while ((1 << Location) & AssignedInputs)
						{
							Location++;
						}
							
						if (Location != Var->location)
						{
							SPVRResult = Reflection.ChangeInputVariableLocation(Var, Location);
							check(SPVRResult == SPV_REFLECT_RESULT_SUCCESS);
						}
							
						uint32 ArrayCount = 1;
						for (uint32 Dim = 0; Dim < Var->array.dims_count; Dim++)
						{
							ArrayCount *= Var->array.dims[Dim];
						}
							
						FString TypeQualifier;

						const SpvReflectTypeDescription& type = *Var->type_description;
						uint32_t masked_type = type.type_flags & 0xF;
							
						switch (masked_type) {
							default: checkf(false, TEXT("unsupported component type %d"), masked_type); break;
							case SPV_REFLECT_TYPE_FLAG_BOOL  : TypeQualifier = TEXT("b"); break;
							case SPV_REFLECT_TYPE_FLAG_INT   : TypeQualifier = (type.traits.numeric.scalar.signedness ? TEXT("i") : TEXT("u")); break;
							case SPV_REFLECT_TYPE_FLAG_FLOAT : TypeQualifier = (type.traits.numeric.scalar.width == 32 ? TEXT("f") : TEXT("h")); break;
						}
							
						if (type.type_flags & SPV_REFLECT_TYPE_FLAG_MATRIX)
						{
							TypeQualifier += FString::Printf(TEXT("%d%d"), type.traits.numeric.matrix.row_count, type.traits.numeric.matrix.column_count);
						}
						else if (type.type_flags & SPV_REFLECT_TYPE_FLAG_VECTOR)
						{
							TypeQualifier += FString::Printf(TEXT("%d"), type.traits.numeric.vector.component_count);
						}
						else
						{
							TypeQualifier += TEXT("1");
						}
							
						for (uint32 j = 0; j < ArrayCount; j++)
						{
							AssignedInputs |= (1 << (Location + j));
								
							CCHeaderWriter.WriteInputAttribute(TEXT("in_ATTRIBUTE"), *TypeQualifier, (Location + j), /*bLocationPrefix:*/ false, /*bLocationSuffix:*/ true);
						}
					}
				}
			}
			
			// Copy reflection code back to SPIR-V buffer
			SpirvData = TArray<uint32>(Reflection.GetCode(), Reflection.GetCodeSize() / sizeof(uint32));
		}
		
		uint32 SideTableIndex = 0;

		CrossCompiler::FShaderConductorTarget TargetDesc;

		if (Result)
		{
			SideTableIndex = FPlatformMath::CountTrailingZeros64(BufferIndices);
			BufferIndices &= ~(1ull << (uint64)SideTableIndex);

			TargetDesc.CompileFlags->SetDefine(TEXT("texel_buffer_texture_width"), 0);
			TargetDesc.CompileFlags->SetDefine(TEXT("enforce_storge_buffer_bounds"), 1);
			TargetDesc.CompileFlags->SetDefine(TEXT("buffer_size_buffer_index"), SideTableIndex);
			TargetDesc.CompileFlags->SetDefine(TEXT("invariant_float_math"), Options.bEnableFMAPass ? 1 : 0);
			TargetDesc.CompileFlags->SetDefine(TEXT("enable_decoration_binding"), 1);

			if(Input.Target.Platform == SP_METAL_SM6)
			{
				// Detect if we need to patch VSM shaders (flatten 2D array as regular 2D texture).
				// Must be done as VSM uses 2DArray and requires atomics support. And Metal does not
				// support atomics on 2Darray...
				TargetDesc.CompileFlags->SetDefine(TEXT("flatten_2d_array"), 1);
				TargetDesc.CompileFlags->SetDefine(TEXT("flatten_2d_array_names"), TEXT("VirtualShadowMap_PhysicalPagePool,OutPhysicalPagePool,OutDepthBufferArray,PhysicalPagePool,ShadowDepthPass_OutDepthBufferArray,PrevAtomicTextureArray,PrevAtomicOutput"));
			}
			else if(Input.Target.Platform == SP_METAL_SM5)
			{
			   TargetDesc.CompileFlags->SetDefine(TEXT("flatten_2d_array"), 1);
			   TargetDesc.CompileFlags->SetDefine(TEXT("flatten_2d_array_names"), TEXT("PrevAtomicTextureArray,PrevAtomicOutput"));
			}

			switch (Semantics)
			{
			case EMetalGPUSemanticsImmediateDesktop:
				TargetDesc.Language = CrossCompiler::EShaderConductorLanguage::Metal_macOS;
				break;
			case EMetalGPUSemanticsTBDRDesktop:
				TargetDesc.Language = CrossCompiler::EShaderConductorLanguage::Metal_iOS;
				TargetDesc.CompileFlags->SetDefine(TEXT("ios_support_base_vertex_instance"), 1);
				TargetDesc.CompileFlags->SetDefine(TEXT("use_framebuffer_fetch_subpasses"), 1);
				TargetDesc.CompileFlags->SetDefine(TEXT("emulate_cube_array"), 1);
				break;
			case EMetalGPUSemanticsMobile:
			default:
				TargetDesc.Language = CrossCompiler::EShaderConductorLanguage::Metal_iOS;
				TargetDesc.CompileFlags->SetDefine(TEXT("ios_support_base_vertex_instance"), 1);
				TargetDesc.CompileFlags->SetDefine(TEXT("use_framebuffer_fetch_subpasses"), 1);
				TargetDesc.CompileFlags->SetDefine(TEXT("emulate_cube_array"), 1);
				break;
			}

			static const TCHAR* subpass_input_dimension_names[] =
			{
				TEXT("subpass_input_dimension0"),
				TEXT("subpass_input_dimension1"),
				TEXT("subpass_input_dimension2"),
				TEXT("subpass_input_dimension3"),
				TEXT("subpass_input_dimension4"),
				TEXT("subpass_input_dimension5"),
				TEXT("subpass_input_dimension6"),
				TEXT("subpass_input_dimension7")
			};

			for (uint32 SubpassIndex = 0; SubpassIndex < MaxMetalSubpasses; SubpassIndex++)
			{
				uint32 SubpassInputDim = SubpassInputsDim[SubpassIndex];
				if (SubpassInputDim >= 1 && SubpassInputDim <= 4)
				{
					// If a dimension for the subpass input attachment at binding slot 0 was determined,
					// forward this dimension to SPIRV-Cross because SPIR-V doesn't support a dimension for OpTypeImage instruction with SubpassData
					TargetDesc.CompileFlags->SetDefine(subpass_input_dimension_names[SubpassIndex], SubpassInputDim);
				}
			}
			
			if (IABTier >= 1)
			{
				TargetDesc.CompileFlags->SetDefine(TEXT("argument_buffers"), 1);
				TargetDesc.CompileFlags->SetDefine(TEXT("argument_buffer_offset"), IABOffsetIndex);
			}
			TargetDesc.CompileFlags->SetDefine(TEXT("texture_buffer_native"), 1);
			
			switch (VersionEnum)
			{
#if PLATFORM_MAC
				case 8:
				{
					TargetDesc.Version = 30000;
					break;
				}
				case 7:
				{
					TargetDesc.Version = 20400;
					break;
				}
				case 6:
				{
					TargetDesc.Version = 20300;
					break;
				}
				case 5:
				{
					TargetDesc.Version = 20200;
					break;
				}
				default:
				{
					UE_LOGF(LogShaders, Warning, "Metal Shader Version Unsupported, switching to default 2.2"); //EMacMetalShaderStandard::MacMetalSLStandard_Minimum
					TargetDesc.Version = 20200;
					break;
				}
#else
				case 7:
				{
					TargetDesc.Version = 20400;
					break;
				}
				case 8:
				{
					TargetDesc.Version = 30000;
					break;
				}
				case 9:
				{
					TargetDesc.Version = 30100;
					break;
				}
				default:
				{
					UE_LOGF(LogShaders, Warning, "Metal Shader Version Unsupported, switching to default 2.4"); //EIOSMetalShaderStandard::IOSMetalSLStandard_Minimum
					TargetDesc.Version = 20400;
					break;
				}
#endif
			}
		}

		// Convert SPIR-V binary to Metal source
		std::string ResultsTargetDataAsString;
		bool bMetalSourceCompileSucceeded = false;

		if (Result)
		{
			bMetalSourceCompileSucceeded = CompilerContext.CompileSpirvToSourceBuffer(
				Options, TargetDesc, SpirvData.GetData(), SpirvData.Num() * sizeof(uint32),
				[&ResultsTargetDataAsString](const void* Data, uint32 Size)
				{
					ResultsTargetDataAsString = std::string(reinterpret_cast<const ANSICHAR*>(Data), Size);
				}
			);
		}

		if (!bMetalSourceCompileSucceeded)
		{
			// Compilation failed.
			Result = 0;
		}
		else
		{
			if (Options.bEnableFMAPass)
			{
				std::string FMADefine = std::string("\n"
										"template<typename T>\n"
										"static inline __attribute__((always_inline))\n"
										"T ue_cross(T x, T y)\n"
										"{\n"
										"    metal::float3 fx = metal::float3(x);\n"
										"    metal::float3 fy = metal::float3(y);\n"
										"    return T(metal::fma(fx[1], fy[2], -metal::fma(fy[1], fx[2], 0.0)), metal::fma(fx[2], fy[0], -metal::fma(fy[2], fx[0], 0.0)), metal::fma(fx[0], fy[1], -metal::fma(fy[0], fx[1], 0.0)));\n"
										"}\n"
										"#define cross ue_cross\n\n"
										"using namespace metal;"
										);
				
				std::string IncludeString = "using namespace metal;";
				size_t IncludePos = ResultsTargetDataAsString.find(IncludeString);
				if (IncludePos != std::string::npos)
					ResultsTargetDataAsString.replace(IncludePos, IncludeString.length(), FMADefine);
			}

			// We calculate the CRC of the metal source code before adding comments
			CRCLen = ResultsTargetDataAsString.length();
			CRC = FCrc::MemCrc_DEPRECATED(ResultsTargetDataAsString.c_str(), CRCLen);
			
			ANSICHAR MainCRC[25];
			int32 NewLen = FCStringAnsi::Snprintf(MainCRC, 25, "Main_%0.8x_%0.8x(", CRCLen, CRC);

			if (FCStringAnsi::Strstr(ResultsTargetDataAsString.c_str(), "spvBufferSizeConstants"))
			{
				CCHeaderWriter.WriteSideTable(TEXT("spvBufferSizeConstants"), SideTableIndex);
			}

			CCHeaderWriter.WriteSourceInfo(*Input.VirtualSourceFilePath, *Input.EntryPointName);
			CCHeaderWriter.WriteCompilerInfo();

			FString MetaData = CCHeaderWriter.ToString();
			MetaData += RTString;
			MetaData += TEXT("\n\n");
			if (ALNString.Len())
			{
				MetaData += TEXT("// Attributes: ");
				MetaData += ALNString;
				MetaData += TEXT("\n\n");
			}
			
			MetalSource = TCHAR_TO_UTF8(*MetaData);
			MetalSource += ResultsTargetDataAsString;

			std::string MainEntryPoint = EntryPointNameAnsi + "(";
			size_t Pos;
			do
			{
				Pos = MetalSource.find(MainEntryPoint);
				if (Pos != std::string::npos)
					MetalSource.replace(Pos, MainEntryPoint.length(), MainCRC);
			} while(Pos != std::string::npos);
		}
		
		// Version 6 means Tier 2 IABs for now.
		if (IABTier >= 2)
		{
			char BufferIdx[3];
			for (auto& IAB : IABs)
			{
				uint32 Index = IAB.Value[0].SetIndex;
				FMemory::Memzero(BufferIdx);
				FCStringAnsi::Snprintf(BufferIdx, 3, "%d", Index);
				std::string find_str = "struct spvDescriptorSetBuffer";
				find_str += BufferIdx;
				size_t Pos = MetalSource.find(find_str);
				if (Pos != std::string::npos)
				{
					size_t StartPos = MetalSource.find("{", Pos);
					size_t EndPos = MetalSource.find("}", StartPos);
					std::string IABName(TCHAR_TO_UTF8(*IAB.Key));
					size_t UBPos = MetalSource.find("constant type_" + IABName + "*");
					
					std::string Declaration = find_str + "\n{\n\tconstant uint* spvBufferSizeConstants [[id(0)]];\n";
					for (FMetalResourceTableEntry& Entry : IAB.Value)
					{
						std::string EntryString;
						std::string Name(TCHAR_TO_UTF8(*Entry.Name));
						switch(Entry.Type)
						{
							case UBMT_TEXTURE:
							case UBMT_RDG_TEXTURE:
							case UBMT_RDG_TEXTURE_SRV:
							case UBMT_RDG_TEXTURE_NON_PIXEL_SRV:
							case UBMT_SRV:
							case UBMT_SAMPLER:
							case UBMT_RDG_BUFFER_SRV:
							case UBMT_UAV:
							case UBMT_RDG_TEXTURE_UAV:
							case UBMT_RDG_BUFFER_UAV:
							{
								size_t EntryPos = MetalSource.find(Name + " [[id(");
								if (EntryPos != std::string::npos)
								{
									while(MetalSource[--EntryPos] != '\n') {}
									while(MetalSource[++EntryPos] != '\n')
									{
										EntryString += MetalSource[EntryPos];
									}
									EntryString += "\n";
								}
								else
								{
									switch(Entry.Type)
									{
										case UBMT_TEXTURE:
										case UBMT_RDG_TEXTURE:
										case UBMT_RDG_TEXTURE_SRV:
										case UBMT_RDG_TEXTURE_NON_PIXEL_SRV:
										case UBMT_SRV:
										{
											std::string typeName = "texture_buffer<float, access::read>";
											int32 NameIndex = PreprocessedShader.Find(Entry.Name + ";");
											int32 DeclIndex = NameIndex;
											if (DeclIndex > 0)
											{
												while(PreprocessedShader[--DeclIndex] != TEXT('\n')) {}
												FString Decl = PreprocessedShader.Mid(DeclIndex, NameIndex - DeclIndex);
												TCHAR const* Types[] = { TEXT("ByteAddressBuffer<"), TEXT("StructuredBuffer<"), TEXT("Buffer<"), TEXT("Texture2DArray"), TEXT("TextureCubeArray"), TEXT("Texture2D"), TEXT("Texture3D"), TEXT("TextureCube") };
												char const* NewTypes[] = { "device void*", "device void*", "texture_buffer<float, access::read>", "texture2d_array<float>", "texturecube_array<float>", "texture2d<float>", "texture3d<float>", "texturecube<float>" };
												for (uint32 i = 0; i < 8; i++)
												{
													if (Decl.Contains(Types[i]))
													{
														typeName = NewTypes[i];
														break;
													}
												}
											}
											
											FCStringAnsi::Snprintf(BufferIdx, 3, "%d", Entry.ResourceIndex + 1);
											EntryString = "\t";
											EntryString += typeName;
											EntryString += " ";
											EntryString += Name;
											EntryString += " [[id(";
											EntryString += BufferIdx;
											EntryString += ")]];\n";
											break;
										}
										case UBMT_SAMPLER:
										{
											FCStringAnsi::Snprintf(BufferIdx, 3, "%d", Entry.ResourceIndex + 1);
											EntryString = "\tsampler ";
											EntryString += Name;
											EntryString += " [[id(";
											EntryString += BufferIdx;
											EntryString += ")]];\n";
											break;
										}
										case UBMT_RDG_BUFFER_SRV:
										{
											FCStringAnsi::Snprintf(BufferIdx, 3, "%d", Entry.ResourceIndex + 1);
											EntryString = "\tdevice void* ";
											EntryString += Name;
											EntryString += " [[id(";
											EntryString += BufferIdx;
											EntryString += ")]];\n";
											break;
										}
										case UBMT_UAV:
										case UBMT_RDG_TEXTURE_UAV:
										{
											std::string typeName = "texture_buffer<float, access::read_write>";
											int32 NameIndex = PreprocessedShader.Find(Entry.Name + ";");
											int32 DeclIndex = NameIndex;
											if (DeclIndex > 0)
											{
												while(PreprocessedShader[--DeclIndex] != TEXT('\n')) {}
												FString Decl = PreprocessedShader.Mid(DeclIndex, NameIndex - DeclIndex);
												TCHAR const* Types[] = { TEXT("ByteAddressBuffer<"), TEXT("StructuredBuffer<"), TEXT("Buffer<"), TEXT("Texture2DArray"), TEXT("TextureCubeArray"), TEXT("Texture2D"), TEXT("Texture3D"), TEXT("TextureCube") };
												char const* NewTypes[] = { "device void*", "device void*", "texture_buffer<float, access::read_write>", "texture2d_array<float, access::read_write>", "texturecube_array<float, access::read_write>", "texture2d<float, access::read_write>", "texture3d<float, access::read_write>", "texturecube<float, access::read_write>" };
												for (uint32 i = 0; i < 8; i++)
												{
													if (Decl.Contains(Types[i]))
													{
														typeName = NewTypes[i];
														break;
													}
												}
											}
											
											FCStringAnsi::Snprintf(BufferIdx, 3, "%d", Entry.ResourceIndex + 1);
											EntryString = "\t";
											EntryString += typeName;
											EntryString += " ";
											EntryString += Name;
											EntryString += " [[id(";
											EntryString += BufferIdx;
											EntryString += ")]];\n";
											
											FCStringAnsi::Snprintf(BufferIdx, 3, "%d", Entry.ResourceIndex + 2);
											EntryString = "\tdevice void* ";
											EntryString += Name;
											EntryString += "_atomic [[id(";
											EntryString += BufferIdx;
											EntryString += ")]];\n";
											break;
										}
										case UBMT_RDG_BUFFER_UAV:
										{
											FCStringAnsi::Snprintf(BufferIdx, 3, "%d", Entry.ResourceIndex + 1);
											EntryString = "\ttexture_buffer<float, access::read_write> ";
											EntryString += Name;
											EntryString += " [[id(";
											EntryString += BufferIdx;
											EntryString += ")]];\n";
											
											FCStringAnsi::Snprintf(BufferIdx, 3, "%d", Entry.ResourceIndex + 2);
											EntryString = "\tdevice void* ";
											EntryString += Name;
											EntryString += "_atomic [[id(";
											EntryString += BufferIdx;
											EntryString += ")]];\n";
											break;
										}
										default:
											break;
									}
								}
								Declaration += EntryString;
								break;
							}
							default:
							{
								break;
							}
						}
					}
					if (UBPos < EndPos)
					{
						size_t UBEnd = MetalSource.find(";", UBPos);
						std::string UBStr = MetalSource.substr(UBPos, (UBEnd - UBPos));
						Declaration += "\t";
						Declaration += UBStr;
						Declaration += ";\n";
					}
					else
					{
						Declaration += "\tconstant void* uniformdata [[id(";
						FMemory::Memzero(BufferIdx);
						FCStringAnsi::Snprintf(BufferIdx, 3, "%d", IAB.Value.Num() + 1);
						Declaration += BufferIdx;
						Declaration += ")]];\n";
					}
					
					Declaration += "}";
					
					MetalSource.replace(Pos, (EndPos - Pos) + 1, Declaration);
				}
			}
		}

		// Flush compile errors
		CompilerContext.FlushErrors(Output.Errors);
	}
#endif
	
	// Attribute [[clang::optnone]] causes performance hit with WPO on M1 Macs => replace with empty space
	const std::string ClangOptNoneString = "[[clang::optnone]]";
	for (size_t Begin = 0, End = 0; (Begin = MetalSource.find(ClangOptNoneString, End)) != std::string::npos; End = Begin)
	{
		MetalSource.replace(Begin, ClangOptNoneString.length(), " ");
	}
	
	if (bOutputAnalysisArtifacts)
	{
		Output.AddStatistic<FString>(TEXT("Metal"), ANSI_TO_TCHAR(&MetalSource[0]), FGenericShaderStat::EFlags::Hidden, FShaderStatTagNames::AnalysisArtifactsName);
	}

	if (bDumpDebugInfo && !MetalSource.empty())
	{
		DumpDebugShaderText(Input, &MetalSource[0], MetalSource.size(), TEXT("metal"));
	}

	if (Result != 0)
	{
		Output.Target = Input.Target;
		BuildMetalShaderOutput(Output,
			Input,
			ShaderParameterParser,
			MetalSource.c_str(),
			MetalSource.length(),
			CRCLen,
			CRC,
			VersionEnum,
			*Standard,
			*MinOSVersion,
			Output.Errors,
			OutputData.TypedBuffers,
			OutputData.InvariantBuffers,
			OutputData.TypedUAVs, 
			OutputData.ConstantBuffers,
			bAllowFastIntrinsics,
			0,
			0,
			0,
			false,
			nullptr,
			FMetalShaderBytecode(),
			nullptr);
	}
	else
	{
		// Log errors on failed compilation in this backend only when compiling from a debug dump USF.
		if (EnumHasAnyFlags(Input.DebugInfoFlags, EShaderDebugInfoFlags::CompileFromDebugUSF))
		{
			for (const FShaderCompilerError& Error : Output.Errors)
			{
				UE_LOGF(LogShaders, Error, "%ls", *Error.GetErrorStringWithLineMarker());
			}
			GLog->Flush();
		}
	}
}
