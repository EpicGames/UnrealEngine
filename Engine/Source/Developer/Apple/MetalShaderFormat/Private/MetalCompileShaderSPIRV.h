// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ShaderCompilerCommon.h"
#include "MetalShaderCompiler.h"
#include "HlslccHeaderWriter.h"

class FMetalCompileShaderSPIRV
{
public:
	static void DoCompileMetalShader(
		const FShaderCompilerInput& Input,
		const FShaderParameterParser& ShaderParameterParser,
		FShaderCompilerOutput& Output,
		const FString& InPreprocessedShader,
		uint32 VersionEnum,
		EMetalGPUSemantics Semantics,
		uint32 MaxUnrollLoops,
		EShaderFrequency Frequency,
		bool bDumpDebugInfo,
		const FString& Standard,
		const FString& MinOSVersion
	);
};
