// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetalShaderCompiler.h"

#include "Containers/AnsiString.h"
#include "Containers/ArrayView.h"
#include "HAL/PlatformFileManager.h"
#include "MetalShaderFormat.h"
#include "MetalShaderResources.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Compression.h"
#include "Misc/EngineVersion.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "ShaderCompilerCommon.h"
#include "ShaderCompilerDefinitions.h"
#include "ShaderCore.h"
#include "ShaderParameterParser.h"
#include "ShaderPreprocessTypes.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "ShaderPlatformConfig.h"

#include "MetalCompileShaderSPIRV.h"
#include "MetalCompileShaderMSC.h"

#if PLATFORM_MAC || PLATFORM_WINDOWS
THIRD_PARTY_INCLUDES_START
#include "metal_irconverter.h"
THIRD_PARTY_INCLUDES_END
#endif

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
	#include <objbase.h>
	#include <assert.h>
	#include <stdio.h>
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"
#endif

#include "ShaderPreprocessor.h"
#include "MetalBackend.h"
#include "MetalShaderCompiler.h"

#include <regex>

#if !PLATFORM_WINDOWS
#if PLATFORM_TCHAR_IS_CHAR16
#define FP_TEXT_PASTE(x) L ## x
#define WTEXT(x) FP_TEXT_PASTE(x)
#else
#define WTEXT TEXT
#endif
#endif

static bool ShouldUseRuntimeShaderCompiling(const FShaderCompilerInput& InputCompilerEnvironment)
{
	const bool bDebug = InputCompilerEnvironment.Environment.CompilerFlags.Contains(CFLAG_Debug);
	return bDebug && InputCompilerEnvironment.Environment.GetCompileArgument(TEXT("METAL_ALLOW_RUNTIME_SHADER_COMPILING"), false);
}

bool ShouldStripIndividualMetalLib(const FShaderCompilerInput& InputCompilerEnvironment)
{	
	return InputCompilerEnvironment.Environment.GetCompileArgument(TEXT("METAL_STRIP_INDIVIDUAL_METALLIBS"), false);
}

constexpr uint16 GMetalMaxUniformBufferSlots = 32;
constexpr int32 GMetalDefaultShadingLanguageVersion = 0;

/*------------------------------------------------------------------------------
	Shader compiling.
------------------------------------------------------------------------------*/

static inline uint32 ParseNumber(const TCHAR* Str)
{
	uint32 Num = 0;
	while (*Str && *Str >= '0' && *Str <= '9')
	{
		Num = Num * 10 + *Str++ - '0';
	}
	return Num;
}

static inline uint32 ParseNumber(const ANSICHAR* Str)
{
	uint32 Num = 0;
	while (*Str && *Str >= '0' && *Str <= '9')
	{
		Num = Num * 10 + *Str++ - '0';
	}
	return Num;
}

struct FHlslccMetalHeader : public CrossCompiler::FHlslccHeader
{
	FHlslccMetalHeader(uint32 const Version);
	virtual ~FHlslccMetalHeader();
	
	// After the standard header, different backends can output their own info
	virtual bool ParseCustomHeaderEntries(const ANSICHAR*& ShaderSource) override;
	
	TMap<uint8, TArray<uint8>> ArgumentBuffers;
	int8 SideTable;
	uint32 RayTracingInstanceIndexBuffer;
	uint32 Version;
};

FHlslccMetalHeader::FHlslccMetalHeader(uint32 const InVersion)
{
	SideTable = -1;
	RayTracingInstanceIndexBuffer = UINT_MAX;
	Version = InVersion;
}

FHlslccMetalHeader::~FHlslccMetalHeader()
{
	
}

bool FHlslccMetalHeader::ParseCustomHeaderEntries(const ANSICHAR*& ShaderSource)
{
#define DEF_PREFIX_STR(Str) \
static const ANSICHAR* Str##Prefix = "// @" #Str ": "; \
static const int32 Str##PrefixLen = FCStringAnsi::Strlen(Str##Prefix)
	DEF_PREFIX_STR(ArgumentBuffers);
	DEF_PREFIX_STR(SideTable);
	DEF_PREFIX_STR(RayTracingInstanceIndexBuffer);
#undef DEF_PREFIX_STR
	
	const ANSICHAR* SideTableString = FCStringAnsi::Strstr(ShaderSource, SideTablePrefix);
	if (SideTableString)
	{
		ShaderSource = SideTableString;
		ShaderSource += SideTablePrefixLen;
		while (*ShaderSource && *ShaderSource != '\n')
		{
			if (*ShaderSource == '(')
			{
				ShaderSource++;
				if (*ShaderSource && *ShaderSource != '\n')
				{
					SideTable = (int8)ParseNumber(ShaderSource);
				}
			}
			else
			{
				ShaderSource++;
			}
		}
		
		if (*ShaderSource && !CrossCompiler::Match(ShaderSource, '\n'))
		{
			return false;
		}
		
		if (SideTable < 0)
		{
			UE_LOGF(LogMetalShaderCompiler, Fatal, "Couldn't parse the SideTable buffer index for bounds checking");
			return false;
		}
	}
	
	const ANSICHAR* RayTracingInstanceIndexBufferString = FCStringAnsi::Strstr(ShaderSource, RayTracingInstanceIndexBufferPrefix);
	if (RayTracingInstanceIndexBufferString)
	{
		ShaderSource += RayTracingInstanceIndexBufferPrefixLen;
		if (!CrossCompiler::ParseIntegerNumber(ShaderSource, RayTracingInstanceIndexBuffer))
		{
			return false;
		}

		if (*ShaderSource && !CrossCompiler::Match(ShaderSource, '\n'))
		{
			return false;
		}
	}

	const ANSICHAR* ArgumentTable = FCStringAnsi::Strstr(ShaderSource, ArgumentBuffersPrefix);
	if (ArgumentTable)
	{
		ShaderSource = ArgumentTable;
		ShaderSource += ArgumentBuffersPrefixLen;
		while (*ShaderSource && *ShaderSource != '\n')
		{
			int32 ArgumentBufferIndex = -1;
			if (!CrossCompiler::ParseIntegerNumber(ShaderSource, ArgumentBufferIndex))
			{
				return false;
			}
			check(ArgumentBufferIndex >= 0);
			
			if (!CrossCompiler::Match(ShaderSource, '['))
			{
				return false;
			}
			
			TArray<uint8> Mask;
			while (*ShaderSource && *ShaderSource != ']')
			{
				int32 MaskIndex = -1;
				if (!CrossCompiler::ParseIntegerNumber(ShaderSource, MaskIndex))
				{
					return false;
				}
				
				check(MaskIndex >= 0);
				Mask.Add((uint8)MaskIndex);
				
				if (!CrossCompiler::Match(ShaderSource, ',') && *ShaderSource != ']')
				{
					return false;
				}
			}
			
			if (!CrossCompiler::Match(ShaderSource, ']'))
			{
				return false;
			}
			
			if (!CrossCompiler::Match(ShaderSource, ',') && *ShaderSource != '\n')
			{
				return false;
			}
			
			ArgumentBuffers.Add((uint8)ArgumentBufferIndex, Mask);
		}
	}
	
	return true;
}

/**
 * Construct the final microcode from the compiled and verified shader source.
 * @param ShaderOutput - Where to store the microcode and parameter map.
 * @param InShaderSource - Metal source with input/output signature.
 * @param SourceLen - The length of the Metal source code.
 */
void BuildMetalShaderOutput(
	FShaderCompilerOutput& ShaderOutput,
	const FShaderCompilerInput& ShaderInput,
	const FShaderParameterParser& ShaderParameterParser,
	const ANSICHAR* InShaderSource,
	uint32 SourceLen,
	uint32 SourceCRCLen,
	uint32 SourceCRC,
	uint32 Version,
	TCHAR const* Standard,
	TCHAR const* MinOSVersion,
	TArray<FShaderCompilerError>& OutErrors,
	uint32 TypedBuffers,
	uint32 InvariantBuffers,
	uint32 TypedUAVs,
	uint32 ConstantBuffers,
	bool bAllowFastIntriniscs,
	uint32 NumCBVs,
    uint32 OutputSizeVS,
    uint32 MaxInputPrimitivesPerMeshThreadgroupGS,
    const bool bUsesDiscard,
    char const* ShaderReflectionJSON,
    FMetalShaderBytecode const& CompiledShaderBytecode,
	FMetalBindlessShaderInfo* ShaderInfo = nullptr
	)
{
	ShaderOutput.bSucceeded = false;
	
	const ANSICHAR* USFSource = InShaderSource;
	
	uint32 NumLines = 0;
	const ANSICHAR* Main = FCStringAnsi::Strstr(USFSource, "Main_");
	while (Main && *Main)
	{
		if (*Main == '\n')
		{
			NumLines++;
		}
		Main++;
	}
	
	FHlslccMetalHeader CCHeader(Version);
	if (!CCHeader.Read(USFSource, SourceLen))
	{
		UE_LOGF(LogMetalShaderCompiler, Fatal, "Bad hlslcc header found");
	}
	
	const EShaderFrequency Frequency = ShaderOutput.Target.GetFrequency();
	const bool bBindlessEnabled = ShaderInput.IsBindlessEnabled();

	const bool bIsMobile = FMetalCompilerToolchain::Get()->IsMobile((EShaderPlatform)ShaderInput.Target.Platform);
	bool bNoFastMath = ShaderInput.Environment.CompilerFlags.Contains(CFLAG_NoFastMath);
	const bool bUsingWPO = ShaderInput.Environment.GetCompileArgument(TEXT("USES_WORLD_POSITION_OFFSET"), false);
	if (bUsingWPO && (ShaderInput.Target.Platform == SP_METAL_SM5_IOS || ShaderInput.Target.Platform == SP_METAL_SM5_TVOS) && Frequency == SF_Vertex)
	{
		// WPO requires that we make all multiply/sincos instructions invariant :(
		bNoFastMath = true;
	}

	FMetalCodeHeader Header;
	FShaderResourceTable SRT;

	Header.CompileFlags = (ShaderInput.Environment.CompilerFlags.Contains(CFLAG_Debug) ? (1 << CFLAG_Debug) : 0);
	Header.CompileFlags |= (bNoFastMath ? (1 << CFLAG_NoFastMath) : 0);
	Header.CompileFlags |= (ShaderInput.Environment.CompilerFlags.Contains(CFLAG_ExtraShaderData) ? (1 << CFLAG_ExtraShaderData) : 0);
	Header.CompileFlags |= (ShaderInput.Environment.CompilerFlags.Contains(CFLAG_ZeroInitialise) ? (1 <<  CFLAG_ZeroInitialise) : 0);
	Header.CompileFlags |= (ShaderInput.Environment.CompilerFlags.Contains(CFLAG_BoundsChecking) ? (1 << CFLAG_BoundsChecking) : 0);
	Header.CompileFlags |= (ShaderInput.Environment.CompilerFlags.Contains(CFLAG_Archive) ? (1 << CFLAG_Archive) : 0);

	Header.Version = Version;
	Header.SideTable = -1;
	Header.SourceLen = SourceCRCLen;
	Header.SourceCRC = SourceCRC;
	Header.Bindings.ConstantBuffers = ConstantBuffers;
    
	FShaderParameterMap& ParameterMap = ShaderOutput.ParameterMap;

	TBitArray<> UsedUniformBufferSlots;
	UsedUniformBufferSlots.Init(false,32);

	// Write out the magic markers.
	Header.Frequency = Frequency;

	// Only inputs for vertex shaders must be tracked.
	if (Frequency == SF_Vertex)
	{
		static const FString AttributePrefix = TEXT("in_ATTRIBUTE");
		for (const CrossCompiler::FHlslccHeader::FInOut& Input : CCHeader.Inputs)
		{
			// Only process attributes.
			if (Input.Name.StartsWith(AttributePrefix))
			{
				uint8 AttributeIndex = ParseNumber(*Input.Name + AttributePrefix.Len());
				Header.Bindings.InOutMask.EnableField(AttributeIndex);
			}
		}
	}

	// Then the list of outputs.
	static const FString TargetPrefix = "FragColor";
	static const FString TargetPrefix2 = "SV_Target";
	static const FString DepthTargetPrefix = "SV_Depth";
	// Only outputs for pixel shaders must be tracked.
	if (Frequency == SF_Pixel)
	{
		for (const CrossCompiler::FHlslccHeader::FInOut& Output : CCHeader.Outputs)
		{
			// Handle targets.
			if (Output.Name.StartsWith(TargetPrefix))
			{
				uint8 TargetIndex = ParseNumber(*Output.Name + TargetPrefix.Len());
				Header.Bindings.InOutMask.EnableField(TargetIndex);
			}
			else if (Output.Name.StartsWith(TargetPrefix2))
			{
				uint8 TargetIndex = ParseNumber(*Output.Name + TargetPrefix2.Len());
				Header.Bindings.InOutMask.EnableField(TargetIndex);
			}
			else if (Output.Name.StartsWith(DepthTargetPrefix))
            {
                Header.Bindings.InOutMask.EnableField(CrossCompiler::FShaderBindingInOutMask::DepthStencilMaskIndex);
            }
		}
		
		// For fragment shaders that discard but don't output anything we need at least a depth-stencil surface, so we need a way to validate this at runtime.
		if (FCStringAnsi::Strstr(USFSource, "[[ depth(") != nullptr || FCStringAnsi::Strstr(USFSource, "[[depth(") != nullptr)
		{
			Header.Bindings.InOutMask.EnableField(CrossCompiler::FShaderBindingInOutMask::DepthStencilMaskIndex);
		}
        
		// For fragment shaders that discard but don't output anything we need at least a depth-stencil surface, so we need a way to validate this at runtime.
		if (FCStringAnsi::Strstr(USFSource, "discard_fragment()") != nullptr)
		{
			EnumAddFlags(Header.Bindings.Flags, EMetalBindingsFlags::PixelDiscard);
		}
	}

	// Then 'normal' uniform buffers.
	bool bOutOfBounds = false;
	for (const CrossCompiler::FHlslccHeader::FAttribute& UniformBlock : CCHeader.UniformBlocks)
	{
		uint16 UBIndex = UniformBlock.Index;
		if (UBIndex >= Header.Bindings.NumUniformBuffers)
		{
			Header.Bindings.NumUniformBuffers = UBIndex + 1;
		}
		if (UBIndex >= GMetalMaxUniformBufferSlots)
		{
			bOutOfBounds = true;
			new(OutErrors) FShaderCompilerError(*FString::Printf(TEXT("Uniform buffer index (%d) exceeded upper bound of slots (%d) for Metal API: %s"), UBIndex, GMetalMaxUniformBufferSlots, *UniformBlock.Name));
			continue;
		}
		UsedUniformBufferSlots[UBIndex] = true;

		HandleReflectedUniformBuffer(ShaderInput, UniformBlock.Name, UBIndex, ShaderOutput);
	}
	
	if (bOutOfBounds)
	{
		ShaderOutput.bSucceeded = false;
		return;
	}

	// Packed global uniforms
	const uint16 BytesPerComponent = 4;
	TMap<ANSICHAR, uint16> PackedGlobalArraySize;
	for (const CrossCompiler::FHlslccHeader::FPackedGlobal& PackedGlobal : CCHeader.PackedGlobals)
	{
		HandleReflectedGlobalConstantBufferMember(
			ShaderInput,
			ShaderParameterParser,
			PackedGlobal.Name,
			PackedGlobal.PackedType,
			PackedGlobal.Offset * BytesPerComponent,
			PackedGlobal.Count * BytesPerComponent,
			ShaderOutput
		);

		uint16& Size = PackedGlobalArraySize.FindOrAdd(PackedGlobal.PackedType);
		Size = FMath::Max<uint16>(BytesPerComponent * (PackedGlobal.Offset + PackedGlobal.Count), Size);
	}
	
	// Packed Uniform Buffers
	TMap<int, TMap<CrossCompiler::EPackedTypeName, uint16> > PackedUniformBuffersSize;
	for (const CrossCompiler::FHlslccHeader::FPackedUB& PackedUB : CCHeader.PackedUBs)
	{
		for (const CrossCompiler::FHlslccHeader::FPackedUB::FMember& Member : PackedUB.Members)
		{
			uint32 ConstantBufferIndex = bBindlessEnabled ? 0 : (uint32)CrossCompiler::EPackedTypeName::HighP;

			// We need to distinguish Root/Global CBs when ShaderConverter is used (mainly for RT support);
			// therefore we perform CB reflection during the previous compilation stage of the pipeline (and keep
			// the vanilla path for SPIRV-Cross).
			if (!bBindlessEnabled)
			{
				HandleReflectedGlobalConstantBufferMember(
					ShaderInput,
					ShaderParameterParser,
					Member.Name,
					ConstantBufferIndex,
					Member.Offset * BytesPerComponent,
					Member.Count * BytesPerComponent,
					ShaderOutput
				);
			}

			uint16& Size = PackedUniformBuffersSize.FindOrAdd(PackedUB.Attribute.Index).FindOrAdd((CrossCompiler::EPackedTypeName)ConstantBufferIndex);
			Size = FMath::Max<uint16>(BytesPerComponent * (Member.Offset + Member.Count), Size);
		}
	}

	// Setup Packed Array info
	Header.Bindings.PackedGlobalArrays.Reserve(PackedGlobalArraySize.Num());
	for (auto Iterator = PackedGlobalArraySize.CreateIterator(); Iterator; ++Iterator)
	{
		ANSICHAR TypeName = Iterator.Key();
		uint16 Size = Iterator.Value();
		Size = (Size + 0xf) & (~0xf);
		CrossCompiler::FPackedArrayInfo Info;
		Info.Size = Size;
		Info.TypeName = TypeName;
		Info.TypeIndex = (uint8)CrossCompiler::PackedTypeNameToTypeIndex(TypeName);
		Header.Bindings.PackedGlobalArrays.Add(Info);
	}

	// In this mode there should only be 0 or 1 packed UB that contains all the aligned & named global uniform parameters
	check(PackedUniformBuffersSize.Num() <= 1);
	for (auto Iterator = PackedUniformBuffersSize.CreateIterator(); Iterator; ++Iterator)
	{
		int BufferIndex = Iterator.Key();
		auto& ArraySizes = Iterator.Value();
		for (auto IterSizes = ArraySizes.CreateIterator(); IterSizes; ++IterSizes)
		{
			CrossCompiler::EPackedTypeName TypeName = IterSizes.Key();
			uint16 Size = IterSizes.Value();
			Size = (Size + 0xf) & (~0xf);
			CrossCompiler::FPackedArrayInfo Info;
			Info.Size = Size;
			Info.TypeName = (ANSICHAR)TypeName;
			Info.TypeIndex = BufferIndex;
			Header.Bindings.PackedGlobalArrays.Add(Info);
		}
	}

	uint32 NumTextures = 0;
	
	// Then samplers.
	TMap<FString, uint32> SamplerMap;
	for (const CrossCompiler::FHlslccHeader::FSampler& Sampler : CCHeader.Samplers)
	{
		HandleReflectedShaderResource(Sampler.Name, Sampler.Offset, Sampler.Count, ShaderOutput);

		NumTextures += Sampler.Count;

		for (auto& SamplerState : Sampler.SamplerStates)
		{
			SamplerMap.Add(SamplerState, Sampler.Count);
		}
	}
	
	Header.Bindings.NumSamplers = CCHeader.SamplerStates.Num();

	// Then UAVs (images in Metal)
	for (const CrossCompiler::FHlslccHeader::FUAV& UAV : CCHeader.UAVs)
	{
		HandleReflectedShaderUAV(UAV.Name, UAV.Offset, UAV.Count, ShaderOutput);

		Header.Bindings.NumUAVs = FMath::Max<uint8>(
			Header.Bindings.NumSamplers,
			UAV.Offset + UAV.Count
			);
	}

	for (const CrossCompiler::FHlslccHeader::FAttribute& SamplerState : CCHeader.SamplerStates)
	{
		if (!SamplerMap.Contains(SamplerState.Name))
		{
			SamplerMap.Add(SamplerState.Name, 1);
		}
		
		HandleReflectedShaderSampler(SamplerState.Name, SamplerState.Index, SamplerMap[SamplerState.Name], ShaderOutput);
	}

    if (bBindlessEnabled)
    {
#if PLATFORM_MAC || PLATFORM_WINDOWS

	#if PLATFORM_MAC
		// Xcode 16 is the minimum supported version for MSC
		if(FPlatformMisc::XCodeVersionCompare(16, 0, 0) < 0)
		{
			UE_LOGF(LogMetalShaderCompiler, Fatal, "Compiling shaders for Metal SM6 requires XCode 16.0 or above");
		}
		
	#endif
		EnumAddFlags(Header.Bindings.Flags, EMetalBindingsFlags::UseMetalShaderConverter);
				
		// Only needed for VS Input (to generate the stage-in function used to convert inputs).
        if (Frequency == SF_Vertex)
        {
            Header.Bindings.IRConverterReflectionJSON = FString(ShaderReflectionJSON);
			check(ShaderReflectionJSON && Header.Bindings.IRConverterReflectionJSON.Len() > 0);
        }
        else
        {
            Header.Bindings.IRConverterReflectionJSON = TEXT("");
        }

		if(ShaderReflectionJSON)
		{
			IRShaderReflectionReleaseString(ShaderReflectionJSON);
		}
		
        Header.Bindings.RSNumCBVs = NumCBVs;
        Header.Bindings.OutputSizeVS = OutputSizeVS;
		if (bUsesDiscard)
		{
			EnumAddFlags(Header.Bindings.Flags, EMetalBindingsFlags::PixelDiscard);
		}
		else
		{
			EnumRemoveFlags(Header.Bindings.Flags, EMetalBindingsFlags::PixelDiscard);
		}

#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
        Header.Bindings.MaxInputPrimitivesPerMeshThreadgroupGS = MaxInputPrimitivesPerMeshThreadgroupGS;
#endif
        
        if (bBindlessEnabled)
        {
            if (Frequency == SF_Pixel)
            {
                // BINDLESS HACK: If the PS writes to UAVs only, we need to set the reflected number
                // of UAVs to a dummy value (to make sure the RHI binds a dummy depth RT).
                if (Header.Bindings.InOutMask.Bitmask == 0)
                {
                    Header.Bindings.NumUAVs = 1;
                }
            }
        }
#else
		UE_LOGF(LogMetalShaderCompiler, Fatal, "Attempting to build using MSC on unsupported platform");
#endif
    }

	Header.NumThreadsX = CCHeader.NumThreads[0];
	Header.NumThreadsY = CCHeader.NumThreads[1];
	Header.NumThreadsZ = CCHeader.NumThreads[2];

	Header.SideTable = CCHeader.SideTable;
	Header.Bindings.ArgumentBufferMasks = CCHeader.ArgumentBuffers;
	Header.Bindings.ArgumentBuffers = 0;
	for (auto const& Pair : Header.Bindings.ArgumentBufferMasks)
	{
		Header.Bindings.ArgumentBuffers |= (1 << Pair.Key);
	}
	
	if (ShaderInput.IsRayTracingShader())
	{
		uint32 RayTracingPayloadType = 0;
		uint32 RayTracingPayloadSize = 0;

		bool bArgFound = ShaderInput.Environment.GetCompileArgument(TEXT("RT_PAYLOAD_TYPE"), RayTracingPayloadType);
		checkf(bArgFound, TEXT("Ray tracing shaders must provide a payload type as this information is required for offline RTPSO compilation. Check that FShaderType::ModifyCompilationEnvironment correctly set this value."));
		bArgFound = ShaderInput.Environment.GetCompileArgument(TEXT("RT_PAYLOAD_MAX_SIZE"), RayTracingPayloadSize);
		checkf(bArgFound, TEXT("Ray tracing shaders must provide a payload size as this information is required for offline RTPSO compilation. Check that FShaderType::ModifyCompilationEnvironment correctly set this value."));

		check(RayTracingPayloadType != 0);
		checkf((FMath::CountBits(RayTracingPayloadType) == 1 && (Header.Frequency == SF_RayHitGroup || Header.Frequency == SF_RayMiss || Header.Frequency == SF_RayCallable)) ||
				(FMath::CountBits(RayTracingPayloadType) >= 1 && (Header.Frequency == SF_RayGen)),
				TEXT("Ray Tracing Shader has %d bits set, which is not the expected count for shader frequency %d"), FMath::CountBits(RayTracingPayloadType), int(Header.Frequency)
		);

		Header.RayTracing.PayloadType = RayTracingPayloadType;
		Header.RayTracing.PayloadSize = RayTracingPayloadSize;
		Header.RayTracing.DefaultBytecodeStage = ShaderInfo->DefaultBytecodeShaderStage;
		Header.RayTracing.ExtendedBytecodeStage = ShaderInfo->ExtendedBytecodeShaderStage;
		Header.RayTracing.EntryPoint = ShaderInfo->UniqueEntryPointName;
		Header.RayTracing.AnyHitEntryPoint = ShaderInfo->UniqueRayAnyHitEntryPoint;
		Header.RayTracing.IntersectionEntryPoint = ShaderInfo->UniqueRayIntersectionEntryPoint;
		Header.RayTracing.GlobalRootParams = ShaderInfo->GlobalRootParams;
		Header.RayTracing.LocalRootParams = ShaderInfo->LocalRootParams;
	}
	
	// Build the SRT for this shader.
	{
		// Build the generic SRT for this shader.
		FShaderCompilerResourceTable GenericSRT;
		BuildResourceTableMapping(ShaderInput.Environment.ResourceTableMap, ShaderInput.Environment.UniformBufferMap, UsedUniformBufferSlots, ShaderOutput.ParameterMap, GenericSRT);
		
		if (ShaderInput.Target.Frequency != SF_RayGen)
		{
			CullGlobalUniformBuffers(ShaderInput.Environment.UniformBufferMap, ShaderOutput.ParameterMap);
		}

		UE::ShaderCompilerCommon::BuildShaderResourceTable(GenericSRT, SRT);

		Header.Bindings.NumUniformBuffers = FMath::Max((uint8)GetNumUniformBuffersUsed(GenericSRT), Header.Bindings.NumUniformBuffers);
	}

	FString MetalCode = FString(USFSource);

	if (ShaderInput.Environment.CompilerFlags.Contains(CFLAG_Debug) && !bBindlessEnabled)
	{
		MetalCode.InsertAt(0, FString::Printf(TEXT("// %s\n"), *CCHeader.Name));
	}
	
	if (Header.Bindings.NumSamplers > MaxMetalSamplers)
	{
		ShaderOutput.bSucceeded = false;
		FShaderCompilerError* NewError = new(OutErrors) FShaderCompilerError();
		
		FString SamplerList;
		for (int32 i = 0; i < CCHeader.SamplerStates.Num(); i++)
		{
			auto const& Sampler = CCHeader.SamplerStates[i];
			SamplerList += FString::Printf(TEXT("%d:%s\n"), Sampler.Index, *Sampler.Name);
		}
		
		NewError->StrippedErrorMessage =
			FString::Printf(TEXT("shader uses %d (%d) samplers exceeding the limit of %d\nSamplers:\n%s"),
				Header.Bindings.NumSamplers, CCHeader.SamplerStates.Num(), MaxMetalSamplers, *SamplerList);
	}
	else if(ShouldUseRuntimeShaderCompiling(ShaderInput) && !bBindlessEnabled)
	{
		// Write out the header and shader source code.
		FMemoryWriter Ar(ShaderOutput.ShaderCode.GetWriteAccess(), true);
		uint8 PrecompiledFlag = 0;
		Ar << PrecompiledFlag;
		Header.Serialize(Ar, SRT);
		Ar.Serialize((void*)USFSource, SourceLen + 1 - (USFSource - InShaderSource));
		
		ShaderOutput.ModifiedShaderSource = MetalCode;
		ShaderOutput.NumInstructions = NumLines;
		ShaderOutput.NumTextureSamplers = Header.Bindings.NumSamplers;
		ShaderOutput.bSucceeded = true;
	}
	else
	{
		FString DebugInfo = TEXT("");
		const bool bGenerateSymbols = (ShaderInput.Environment.CompilerFlags.Contains(CFLAG_GenerateSymbols));
		if (ShaderInput.Environment.CompilerFlags.Contains(CFLAG_ExtraShaderData) || bGenerateSymbols)
		{
			// Include line number and disable optimization
			DebugInfo = TEXT("-gline-tables-only -MO");
			if (bGenerateSymbols)
			{
				// Include source inside the air/metallib
				DebugInfo += TEXT(" -frecord-sources");
			}
		}
		
		FString MathMode = bNoFastMath ? TEXT("-fno-fast-math") : TEXT("-ffast-math");
        
		// at this point, the shader source is ready to be compiled
		// we will need a working temp directory.
		const FString& TempDir = FMetalCompilerToolchain::Get()->GetLocalTempDir();
		
		int32 ReturnCode = 0;
		FString Results;
		FString Errors;
		bool bSucceeded = false;

		EShaderPlatform ShaderPlatform = EShaderPlatform(ShaderInput.Target.Platform);
		const bool bMetalCompilerAvailable = FMetalCompilerToolchain::Get()->IsCompilerAvailable();
		
		FMetalShaderBytecode Bytecode;
		FMetalShaderBytecode ExtendedBytecode;
		
		FString HashedName = FString::Printf(TEXT("%u_%u"), SourceCRCLen, SourceCRC);
		
		if(!bMetalCompilerAvailable)
		{
			// No Metal Compiler - just put the source code directly into /tmp and report error - we are now using text shaders when this was not the requested configuration
			// Move it into place using an atomic move - ensures only one compile "wins"
			FString InputFilename = (TempDir / HashedName) + FMetalCompilerToolchain::MetalExtention;
			FString SaveFile = FPaths::CreateTempFilename(*TempDir, TEXT("ShaderTemp"), TEXT(""));
			FFileHelper::SaveStringToFile(MetalCode, *SaveFile);
			IFileManager::Get().Move(*InputFilename, *SaveFile, false, false, true, true);
			IFileManager::Get().Delete(*SaveFile);
			
			TCHAR const* Message = nullptr;
			if (PLATFORM_MAC)
			{
				Message = TEXT("Xcode's metal shader compiler was not found, verify Xcode has been installed on this Mac and that it has been selected in Xcode > Preferences > Locations > Command-line Tools.");
			}
			
			FShaderCompilerError* Error = new(OutErrors) FShaderCompilerError();
			Error->ErrorVirtualFilePath = InputFilename;
			Error->ErrorLineString = TEXT("0");
			Error->StrippedErrorMessage = FString(Message);
		}
		else if (bBindlessEnabled)
        {
            // The base name (which is <temp>/CRCHash_Length)
            FString BaseFileName = FPaths::Combine(TempDir, HashedName);
            FString MetalFileName = BaseFileName + FMetalCompilerToolchain::MetalExtention;

			check(ShaderInfo);
			
            Bytecode.NativePath = MetalFileName;
            Bytecode.ObjectFile = ShaderInfo->MetalBytecode.ObjectFile;
            Bytecode.OutputFile = ShaderInfo->MetalBytecode.OutputFile;
			
			ExtendedBytecode.NativePath = MetalFileName;
			ExtendedBytecode.ObjectFile = ShaderInfo->MetalBytecodeExtended.ObjectFile;
			ExtendedBytecode.OutputFile = ShaderInfo->MetalBytecodeExtended.OutputFile;

            bSucceeded = true;
        }
		else
		{
			// Compiler available - more intermediate files will be created. 
			// TODO How to handle multiple streams on the same machine? Needs more uniqueness in the temp dir
			const FString& CompilerVersionString = FMetalCompilerToolchain::Get()->GetCompilerVersionString(ShaderPlatform);
			
			// The base name (which is <temp>/CRCHash_Length)
			FString BaseFileName = FPaths::Combine(TempDir, HashedName);
			// The actual metal shadertext, a .metal file
			FString MetalFileName = BaseFileName + FMetalCompilerToolchain::MetalExtention;
			// The compiled shader, as AIR. An .air file.
			FString AIRFileName = BaseFileName + FMetalCompilerToolchain::MetalObjectExtension;
			// A metallib containing just this shader (gross). A .metallib file.
			FString MetallibFileName = BaseFileName + FMetalCompilerToolchain::MetalLibraryExtension;

			// 'MetalCode' contains the cross compiled MetalSL version of the shader.
			// Save it out.
			{
				// This is the previous behavior, which attempts an atomic move in case there is a race here.
				// TODO can we ever actually race on this? TempDir should be process specific now.
				FString SaveFile = FPaths::CreateTempFilename(*TempDir, TEXT("ShaderTemp"), TEXT(""));
				bool bSuccess = FFileHelper::SaveStringToFile(MetalCode, *SaveFile);
				if (!bSuccess)
				{
					UE_LOGF(LogMetalShaderCompiler, Fatal, "Failed to write Metal shader out to %ls\nShaderText:\n%ls", *SaveFile, *MetalCode);
				}
				bSuccess = IFileManager::Get().Move(*MetalFileName, *SaveFile, false, false, true, true);
				if (!bSuccess && !FPaths::FileExists(MetalFileName))
				{
					UE_LOGF(LogMetalShaderCompiler, Fatal, "Failed to move %ls to %ls", *SaveFile, *MetalFileName);
				}

				if (FPaths::FileExists(SaveFile))
				{
					IFileManager::Get().Delete(*SaveFile);
				}
			}

			// For iOS 14.0+ this is required. Version==0 is IOSMetalSLStandard_Minimum
			const bool bPreserveInvariance = Frequency == SF_Vertex && (Version == 0 || Version > 5);
			
			// TODO This is the actual MetalSL -> AIR piece
			FMetalShaderBytecodeJob Job;
			Job.IncludeDir = TempDir;
			Job.ShaderFormat = ShaderInput.ShaderFormat;
			Job.TmpFolder = TempDir;
			Job.InputFile = MetalFileName;
			Job.OutputFile = MetallibFileName;
			Job.OutputObjectFile = AIRFileName;
			Job.CompilerVersion = CompilerVersionString;
			Job.MinOSVersion = MinOSVersion;
			Job.PreserveInvariance = bPreserveInvariance ? TEXT("-fpreserve-invariance") : TEXT("");
			Job.DebugInfo = DebugInfo;
			Job.MathMode = MathMode;
			Job.Standard = Standard;
			Job.SourceCRCLen = SourceCRCLen;
			Job.SourceCRC = SourceCRC;
			Job.bUseNativeShaderLibrary = ShaderInput.Environment.CompilerFlags.Contains(CFLAG_Archive);
			Job.bOptimizeForSize = ShaderInput.Environment.GetCompileArgument(TEXT("METAL_OPTIMIZE_FOR_SIZE"), false);
			Job.bStripIndividualMetalLibs = ShouldStripIndividualMetalLib(ShaderInput);
			Job.bCompileAsPCH = false;
			Job.ReturnCode = 0;
			
			// Module cache path to ensure that we don't have clashes across multiple instances.
			FString InputHashString;
			BytesToHex(ShaderInput.Hash.GetBytes(), 8, InputHashString);
			const FString& ModulesCachePath = TempDir / TEXT("ModuleCache") / InputHashString;
			Job.ModuleCacheDirectory = ModulesCachePath;
			
			bSucceeded = FMetalCompilerToolchain::Get()->CompileMetalShader(Job, Bytecode);
			if (!bSucceeded)
			{
				FShaderCompilerError* Error = new(OutErrors) FShaderCompilerError();
				Error->ErrorVirtualFilePath = MetalFileName;
				Error->ErrorLineString = TEXT("0");
				Error->StrippedErrorMessage = Job.Message;
			}
			// Clean up the .metal file after building it
			IFileManager::Get().Delete(*MetalFileName);
		}
		
		if (bSucceeded)
		{
			// Write out the header and compiled shader code
			FMemoryWriter Ar(ShaderOutput.ShaderCode.GetWriteAccess(), true);
			uint8 PrecompiledFlag = 1;
			Ar << PrecompiledFlag;
            Header.Serialize(Ar, SRT);

			// jam it into the output bytes
			Ar.Serialize(Bytecode.OutputFile.GetData(), Bytecode.OutputFile.Num());

			if (ShaderInput.Environment.CompilerFlags.Contains(CFLAG_Archive))
			{
				ShaderOutput.ShaderCode.AddOptionalData(EShaderOptionalDataKey::ObjectFile, Bytecode.ObjectFile.GetData(), Bytecode.ObjectFile.Num());
			}
			
			if(ExtendedBytecode.OutputFile.Num())
			{
				ShaderOutput.ShaderCode.AddOptionalData(EShaderOptionalDataKey::VendorExtension, ExtendedBytecode.OutputFile.GetData(), ExtendedBytecode.OutputFile.Num());
			}
			
			if (ShaderInput.Environment.CompilerFlags.Contains(CFLAG_ExtraShaderData) && !ExtendedBytecode.OutputFile.Num())
			{
				// store data we can pickup later with ShaderCode.FindOptionalData(FShaderCodeName::Key), could be removed for shipping
				ShaderOutput.ShaderCode.AddOptionalData(FShaderCodeName::Key, TCHAR_TO_UTF8(*ShaderInput.GenerateShaderName()));
				ShaderOutput.ShaderCode.AddOptionalData(EShaderOptionalDataKey::SourceCode, TCHAR_TO_UTF8(*MetalCode));
				ShaderOutput.ShaderCode.AddOptionalData(EShaderOptionalDataKey::NativePath, TCHAR_TO_UTF8(*Bytecode.NativePath));
			}
			
			ShaderOutput.NumTextureSamplers = Header.Bindings.NumSamplers;
		}

		ShaderOutput.ModifiedShaderSource = MetalCode;
		if(!bBindlessEnabled)
		{
			ShaderOutput.NumInstructions = NumLines;
		}
		ShaderOutput.bSucceeded = bSucceeded;
	}
}

/*------------------------------------------------------------------------------
	External interface.
------------------------------------------------------------------------------*/

bool PreprocessMetalShader(const FShaderCompilerInput& Input, const FShaderCompilerEnvironment& Environment, FShaderPreprocessOutput& PreprocessOutput)
{
	const EShaderFrequency Frequency = (EShaderFrequency)Input.Target.Frequency;
	if (!(Frequency == SF_Vertex 
		  || Frequency == SF_Pixel 
		  || Frequency == SF_Compute
          || Frequency == SF_Geometry
          || Frequency == SF_Mesh
          || Frequency == SF_Amplification
		  || Frequency == SF_RayGen
		  || Frequency == SF_RayMiss
		  || Frequency == SF_RayHitGroup
		  || Frequency == SF_RayCallable
          ))
	{
		PreprocessOutput.LogError(FString::Printf(
			TEXT("%s shaders not supported for use in Metal."),
			CrossCompiler::GetFrequencyName(Frequency)
			));
		return false;
	}

	return UE::ShaderCompilerCommon::ExecuteShaderPreprocessingSteps(PreprocessOutput, Input, Environment);
}

void CompileMetalShader(const FShaderCompilerInput& Input, const FShaderPreprocessOutput& InPreprocessOutput, FShaderCompilerOutput& Output)
{
	FString EntryPointName = Input.EntryPointName;
	FString PreprocessedSource(InPreprocessOutput.GetSourceViewWide());

	FShaderParameterParser::FPlatformConfiguration PlatformConfiguration;
	FShaderParameterParser ShaderParameterParser(PlatformConfiguration);
	if (!ShaderParameterParser.ParseAndModify(Input, Output.Errors, PreprocessedSource))
	{
		// The FShaderParameterParser will add any relevant errors.
		return;
	}

	if (ShaderParameterParser.DidModifyShader())
	{
		Output.ModifiedShaderSource = PreprocessedSource;
	}

	EMetalGPUSemantics Semantics = EMetalGPUSemanticsMobile;
	if (Input.ShaderFormat == NAME_SF_METAL_SM5_IOS
		|| Input.ShaderFormat == NAME_SF_METAL_SM5_TVOS)
	{
		Semantics = EMetalGPUSemanticsTBDRDesktop;
	}
	else if (Input.ShaderFormat == NAME_SF_METAL_ES3_1
		|| Input.ShaderFormat == NAME_SF_METAL_SM5
		|| Input.ShaderFormat == NAME_SF_METAL_SM6)
	{
		Semantics = EMetalGPUSemanticsImmediateDesktop;
	}

	uint32 VersionEnum = GMetalDefaultShadingLanguageVersion;
	bool bFoundVersion = Input.Environment.GetCompileArgument(TEXT("SHADER_LANGUAGE_VERSION"), VersionEnum);
	if (!bFoundVersion)
	{
		new(Output.Errors) FShaderCompilerError(*FString::Printf(TEXT("Missing SHADER_LANGUAGE_VERSION compile argument; Falling back to default value %d"), GMetalDefaultShadingLanguageVersion));
	}

	// TODO read from toolchain
	const bool bIsMobile = FMetalCompilerToolchain::Get()->IsMobile((EShaderPlatform)Input.Target.Platform);
	const bool bAppleTV = (Input.ShaderFormat == NAME_SF_METAL_ES3_1_TVOS || Input.ShaderFormat == NAME_SF_METAL_SM5_TVOS);
	const bool bIsSimulator = (Input.ShaderFormat == NAME_SF_METAL_SIM);

	FString MinOSVersion;
	FString StandardVersion;
	
	switch (VersionEnum)
	{
	case 9:
		StandardVersion = TEXT("3.1");
		if (bAppleTV)
		{
			MinOSVersion = TEXT("-mtvos-version-min=17.0");
		}
		else if (bIsMobile)
		{
			if (bIsSimulator)
			{
				MinOSVersion = TEXT("-miphonesimulator-version-min=17.0");
			}
			else
			{
				MinOSVersion = TEXT("-mios-version-min=17.0");
			}
		}
		else
		{
			MinOSVersion = TEXT("-mmacosx-version-min=14");
		}
		break;
	case 8:
		StandardVersion = TEXT("3.0");
		if (bAppleTV)
		{
			MinOSVersion = TEXT("-mtvos-version-min=16.0");
		}
		else if (bIsMobile)
		{
			if (bIsSimulator)
			{
				MinOSVersion = TEXT("-miphonesimulator-version-min=16.0");
			}
			else
			{
				MinOSVersion = TEXT("-mios-version-min=16.0");
			}
		}
		else
		{
			MinOSVersion = TEXT("-mmacosx-version-min=13");
		}
		break;

	case 7:
		StandardVersion = TEXT("2.4");
		if (bAppleTV)
		{
			MinOSVersion = TEXT("-mtvos-version-min=15.0");
		}
		else if (bIsMobile)
		{
			if (bIsSimulator)
			{
				MinOSVersion = TEXT("-miphonesimulator-version-min=15.0");
			}
			else
			{
				MinOSVersion = TEXT("-mios-version-min=15.0");
			}
		}
		else
		{
			MinOSVersion = TEXT("-mmacosx-version-min=12");
		}
		break;
	case 6:
		StandardVersion = TEXT("2.4");
		if (bAppleTV)
		{
			MinOSVersion = TEXT("-mtvos-version-min=15.0");
		}
		else if (bIsMobile)
		{
			if (bIsSimulator)
			{
				MinOSVersion = TEXT("-miphonesimulator-version-min=15.0");
			}
			else
			{
				MinOSVersion = TEXT("-mios-version-min=15.0");
			}
		}
		else
		{
			// TODO - This is a workaround for an issue with the Apple Shader Compiler
			// leading to corruption on M1/AMD when > 2.3 versions are used. 
			// This should be bumped to 2.4 after it's resolved
			StandardVersion = TEXT("2.3");
			MinOSVersion = TEXT("-mmacosx-version-min=11");
		}
		break;
	case 5:
		// Fall through
	case 0:
		StandardVersion = TEXT("2.4");
		if (bAppleTV)
		{
			MinOSVersion = TEXT("-mtvos-version-min=15.0");
		}
		else if (bIsMobile)
		{
			if (bIsSimulator)
			{
				MinOSVersion = TEXT("-miphonesimulator-version-min=15.0");
			}
			else
			{
				MinOSVersion = TEXT("-mios-version-min=15.0");
			}
		}
		else
		{
			// TODO - This is a workaround for an issue with the Apple Shader Compiler
			// leading to corruption on M1/AMD when > 2.3 versions are used. 
			// This should be bumped to 2.4 after it's resolved
			StandardVersion = TEXT("2.2");
			MinOSVersion = TEXT("-mmacosx-version-min=10.15");
		}
		//EIOSMetalShaderStandard::IOSMetalSLStandard_Minimum
		break;
	default:
		Output.bSucceeded = false;
		{
			FString EngineIdentifier = FEngineVersion::Current().ToString(EVersionComponent::Minor);
			FShaderCompilerError* NewError = new(Output.Errors) FShaderCompilerError();
			NewError->StrippedErrorMessage = FString::Printf(TEXT("Minimum Metal Version is 2.4 in UE %s"), *EngineIdentifier);
			return;
		}
	}

	TCHAR const* StandardPlatform = bIsMobile ? TEXT("ios") : TEXT("macos");
	FString Standard;
	if (VersionEnum >= 8) //-V547
	{
		Standard = FString::Printf(TEXT("-std=metal%s"), *StandardVersion);
	}
	else
	{
		Standard = FString::Printf(TEXT("-std=%s-metal%s"), StandardPlatform, *StandardVersion);
	}

	if (EnumHasAnyFlags(Input.DebugInfoFlags, EShaderDebugInfoFlags::CompileFromDebugUSF))
	{
		// force debug output on when compiling a debug dump usf
		const_cast<FShaderCompilerInput&>(Input).DumpDebugInfoPath = FPaths::GetPath(Input.VirtualSourceFilePath);
	}

	const bool bDumpDebugInfo = Input.DumpDebugInfoEnabled();

	// Allow the shader pipeline to override the platform default in here.
	uint32 MaxUnrollLoops = 32;
	if (Input.Environment.CompilerFlags.Contains(CFLAG_AvoidFlowControl))
	{
		MaxUnrollLoops = 1024; // Max. permitted by hlslcc
	}
	else if (Input.Environment.CompilerFlags.Contains(CFLAG_PreferFlowControl))
	{
		MaxUnrollLoops = 0;
	}

	FMetalBindlessShaderInfo BindlessShaderInfo;
	if (Input.IsBindlessEnabled())
	{
		FMetalCompileShaderMSC::DoCompileMetalShader(Input, ShaderParameterParser, Output, PreprocessedSource, VersionEnum, Semantics, MaxUnrollLoops, (EShaderFrequency)Input.Target.Frequency, bDumpDebugInfo, Standard, MinOSVersion, BindlessShaderInfo);
	}
	else
	{
		FMetalCompileShaderSPIRV::DoCompileMetalShader(Input, ShaderParameterParser, Output, PreprocessedSource, VersionEnum, Semantics, MaxUnrollLoops, (EShaderFrequency)Input.Target.Frequency, bDumpDebugInfo, Standard, MinOSVersion);
	}
	ShaderParameterParser.ValidateShaderParameterTypes(Input, bIsMobile, Output);
}

bool StripShader_Metal(TArray<uint8>& Code, class FString const& DebugPath, bool const bNative)
{
	bool bSuccess = false;
	
	FShaderCodeReader ShaderCode(Code);
	FMemoryReader Ar(Code, true);
	Ar.SetLimitSize(ShaderCode.GetActualShaderCodeSize());
	
	// was the shader already compiled offline?
	uint8 OfflineCompiledFlag;
	Ar << OfflineCompiledFlag;
	
	if(bNative && OfflineCompiledFlag == 1)
	{
		// get the header
		FMetalCodeHeader Header;
        FShaderResourceTable SRT;
        Header.Serialize(Ar, SRT);
		
		const FString ShaderName = ShaderCode.FindOptionalData(FShaderCodeName::Key);

		// Must be compiled for archiving or something is very wrong.
		if(bNative == false || Header.CompileFlags & (1 << CFLAG_Archive))
		{
			bSuccess = true;
			
			// remember where the header ended and code (precompiled or source) begins
			int32 CodeOffset = Ar.Tell();
			const uint8* SourceCodePtr = (uint8*)Code.GetData() + CodeOffset;
			
			// Copy the non-optional shader bytecode
			TArray<uint8> SourceCode;
			SourceCode.Append(SourceCodePtr, ShaderCode.GetActualShaderCodeSize() - CodeOffset);
			
			const ANSICHAR* ShaderSource = ShaderCode.FindOptionalData(EShaderOptionalDataKey::SourceCode);
			
			const size_t ShaderSourceLength = ShaderSource ? FCStringAnsi::Strlen(ShaderSource) : 0;
			bool const bHasShaderSource = ShaderSourceLength > 0;
			
			const ANSICHAR* ShaderPath = ShaderCode.FindOptionalData(EShaderOptionalDataKey::NativePath);
			bool const bHasShaderPath = (ShaderPath && FCStringAnsi::Strlen(ShaderPath) > 0);
			
			if (bHasShaderSource && bHasShaderPath)
			{
				FString DebugFilePath = DebugPath / FString(ShaderPath);
				FString DebugFolderPath = FPaths::GetPath(DebugFilePath);
				if (IFileManager::Get().MakeDirectory(*DebugFolderPath, true))
				{
					FString TempPath = FPaths::CreateTempFilename(*DebugFolderPath, TEXT("MetalShaderFile-"), TEXT(".metal"));
					IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
					IFileHandle* FileHandle = PlatformFile.OpenWrite(*TempPath);
					if (FileHandle)
					{
						FileHandle->Write((const uint8 *)ShaderSource, ShaderSourceLength);
						delete FileHandle;

						IFileManager::Get().Move(*DebugFilePath, *TempPath, true, false, true, false);
						IFileManager::Get().Delete(*TempPath);
					}
					else
					{
						UE_LOGF(LogShaders, Error, "Shader stripping failed: shader %ls (Len: %0.8x, CRC: %0.8x) failed to create file %ls!", *ShaderName, Header.SourceLen, Header.SourceCRC, *TempPath);
					}
				}
			}
			
			if (bNative)
			{
				int32 ObjectSize = 0;
				const uint8* ShaderObject = ShaderCode.FindOptionalDataAndSize(EShaderOptionalDataKey::ObjectFile, ObjectSize);
				
				// If ShaderObject and ObjectSize is zero then the code has already been stripped - source code should be the byte code
				if(ShaderObject && ObjectSize)
				{
					TArray<uint8> ObjectCodeArray;
					ObjectCodeArray.Append(ShaderObject, ObjectSize);
					SourceCode = ObjectCodeArray;
				}
			}
			
			// We store the extra code in the SourceCode section temporarily
			static constexpr const EShaderOptionalDataKey ExtendedCodeSection = EShaderOptionalDataKey::VendorExtension;
						
			// Include optional shader data sections if H or I is present (to include extended source code for shaders with complex entrypoints; e.g. RT shaders).
			int32 ExtendedCodeDataSize = -1;
			uint8 const* ExtendedCodeArch = nullptr; 
			
			if(IsRayTracingShaderFrequency((EShaderFrequency)Header.Frequency))
			{
				ExtendedCodeArch = ShaderCode.FindOptionalDataAndSize(ExtendedCodeSection, ExtendedCodeDataSize);
			}

			if (ExtendedCodeArch && ExtendedCodeDataSize > 0)
			{
				Header.RayTracing.ExtendedCodeDataSize = ExtendedCodeDataSize;
				Header.RayTracing.ExtendedCodeDataOffset = SourceCode.Num();
				SourceCode.Append(ExtendedCodeArch, ExtendedCodeDataSize);
			}

			// Strip any optional data
			if (bNative || ShaderCode.GetOptionalDataSize() > 0)
			{
				// Write out the header and compiled shader code
				FShaderCode NewCode;
				FMemoryWriter NewAr(NewCode.GetWriteAccess(), true);
				NewAr << OfflineCompiledFlag;
                Header.Serialize(NewAr, SRT);
				
				// jam it into the output bytes
				NewAr.Serialize(SourceCode.GetData(), SourceCode.Num());
				
				Code = NewCode.GetReadView();
			}
		}
		else
		{
			UE_LOGF(LogShaders, Error, "Shader stripping failed: shader %ls (Len: %0.8x, CRC: %0.8x) was not compiled for archiving into a native library (Native: %ls, Compile Flags: %0.8x)!", *ShaderName, Header.SourceLen, Header.SourceCRC, bNative ? TEXT("true") : TEXT("false"), (uint32)Header.CompileFlags);
		}
	}
	else
	{
		UE_LOGF(LogShaders, Error, "Shader stripping failed: shader %ls (Native: %ls, Offline Compiled: %d) was not compiled to bytecode for native archiving!", *DebugPath, bNative ? TEXT("true") : TEXT("false"), OfflineCompiledFlag);
	}
	
	return bSuccess;
}

uint64 AppendShader_Metal(FString const& WorkingDir, const FShaderHash& Hash, TArray<uint8>& InShaderCode)
{
	uint64 Id = 0;
	
	const bool bCompilerAvailable = FMetalCompilerToolchain::Get()->IsCompilerAvailable();
	if (bCompilerAvailable)
	{
		// Parse the existing data and extract the source code. We have to recompile it
		FShaderCodeReader ShaderCode(InShaderCode);
		FMemoryReader Ar(InShaderCode, true);
		Ar.SetLimitSize(ShaderCode.GetActualShaderCodeSize());
		
		// was the shader already compiled offline?
		uint8 OfflineCompiledFlag;
		Ar << OfflineCompiledFlag;
		if (OfflineCompiledFlag == 1)
		{
			// get the header
            FMetalCodeHeader Header;
            FShaderResourceTable SRT;
            Header.Serialize(Ar, SRT);

			const FString ShaderName = ShaderCode.FindOptionalData(FShaderCodeName::Key);

			// Must be compiled for archiving or something is very wrong.
			if(Header.CompileFlags & (1 << CFLAG_Archive))
			{
				// remember where the header ended and code (precompiled or source) begins
				int32 CodeOffset = Ar.Tell();
				const uint8* SourceCodePtr = (uint8*)InShaderCode.GetData() + CodeOffset;
				
				// Copy the non-optional shader bytecode
				int32 ObjectCodeDataSize = 0;
				uint8 const* Object = ShaderCode.FindOptionalDataAndSize(EShaderOptionalDataKey::ObjectFile, ObjectCodeDataSize);

				// 'o' segment missing this is a pre stripped shader
				if(!Object)
				{
					ObjectCodeDataSize = ShaderCode.GetActualShaderCodeSize() - CodeOffset;
					Object = SourceCodePtr;
				}
				
				// For archiving, we need to move the extended section to a separate object file
				// (as cooked builds do a library lookup; rather than creating a new one from binary).
				bool bHasExtendedObjectFile = false;
				FString ExtendedObjFilename;
				
				if (Header.RayTracing.ExtendedCodeDataSize > 0)
				{
					ExtendedObjFilename = WorkingDir / FString::Printf(TEXT("Ex_%s_%0.8x_%0.8x.o"), *Header.RayTracing.EntryPoint, Header.SourceLen, Header.SourceCRC);
					bHasExtendedObjectFile = IFileManager::Get().FileExists(*ExtendedObjFilename);
					
					const uint8* ExtendedCodeArchPtr = SourceCodePtr + Header.RayTracing.ExtendedCodeDataOffset;

					check(Header.RayTracing.ExtendedCodeDataSize > 0);
					check(ExtendedCodeArchPtr);

					TArrayView<const uint8> ExtendedObjectCodeArray(ExtendedCodeArchPtr, Header.RayTracing.ExtendedCodeDataSize);

					// write out shader object code source (IR) for archiving to a single library file later
					if( FFileHelper::SaveArrayToFile(ExtendedObjectCodeArray, *ExtendedObjFilename) )
					{
						bHasExtendedObjectFile = true;
					}
				}
				
				TArrayView<const uint8> ObjectCodeArray(Object, ObjectCodeDataSize);
				
				// Object code segment
				FString ObjFilename = WorkingDir / FString::Printf(TEXT("Main_%0.8x_%0.8x.o"), Header.SourceLen, Header.SourceCRC);

				bool bHasObjectFile = IFileManager::Get().FileExists(*ObjFilename);
				bool const bHasObjectData = (ObjectCodeDataSize > 0) || bHasObjectFile;
				
				if (bHasObjectData)
				{
					// metal commandlines
					int32 ReturnCode = 0;
					FString Results;
					FString Errors;
						
					if (bHasExtendedObjectFile)
					{
						// Write the main lib to a temporary location.
						FString TmpObjFilename = WorkingDir / FString::Printf(TEXT("Main_%s_%0.8x_%0.8x.o"), *Header.RayTracing.EntryPoint, Header.SourceLen, Header.SourceCRC);
						if( FFileHelper::SaveArrayToFile(ObjectCodeArray, *TmpObjFilename) )
						{
							bHasObjectFile = true;
						}
						check(bHasObjectFile);

						const TCHAR* SourcePackingOption = TEXT("-pack-sources internal");
						FString AirPackParams = FString::Printf(TEXT("-pack-metallibs internal -pack-descriptors internal -pack-reflections none %s \"%s\" \"%s\" -o \"%s\""), SourcePackingOption, *TmpObjFilename, *ExtendedObjFilename, *ObjFilename);

						const FMetalCompilerToolchain* Toolchain = FMetalCompilerToolchain::Get();

						bool bSuccess = Toolchain->ExecMetalPack(AppleSDKMac, *AirPackParams, &ReturnCode, &Results, &Errors);

						// handle compile error
						if (!bSuccess || ReturnCode != 0)
						{
							UE_LOGF(LogShaders, Error, "Archiving failed: air-pack failed with code %d: %ls %ls", ReturnCode, *Results, *Errors);

							// Make sure we propagate failure to the caller.
							bHasObjectFile = false;
						}
					}
					else if (ObjectCodeDataSize > 0)
					{
						// write out shader object code source (IR) for archiving to a single library file later
						if( FFileHelper::SaveArrayToFile(ObjectCodeArray, *ObjFilename) )
						{
							bHasObjectFile = true;
						}
					}
					
					if (bHasObjectFile)
					{
						Id = ((uint64)Header.SourceLen << 32) | Header.SourceCRC;
						
						// This is going to get serialised into the shader resource archive we don't anything but the header info now with the archive flag set
						Header.CompileFlags |= (1 << CFLAG_Archive);
						
						// Write out the header and compiled shader code
						FShaderCode NewCode;
						FMemoryWriter NewAr(NewCode.GetWriteAccess(), true);
						NewAr << OfflineCompiledFlag;
                        Header.Serialize(NewAr, SRT);
						
						InShaderCode = NewCode.GetReadView();
						
						UE_LOGF(LogShaders, Verbose, "Archiving succeeded: shader %ls (Len: %0.8x, CRC: %0.8x, SHA: %ls)", *ShaderName, Header.SourceLen, Header.SourceCRC, *Hash.ToString());
					}
					else
					{
						UE_LOGF(LogShaders, Error, "Archiving failed: failed to write temporary file %ls for shader %ls (Len: %0.8x, CRC: %0.8x, SHA: %ls)", *ObjFilename, *ShaderName, Header.SourceLen, Header.SourceCRC, *Hash.ToString());
					}
				}
				else
				{
					UE_LOGF(LogShaders, Error, "Archiving failed: shader %ls (Len: %0.8x, CRC: %0.8x, SHA: %ls) has no object data", *ShaderName, Header.SourceLen, Header.SourceCRC, *Hash.ToString());
				}
			}
			else
			{
				UE_LOGF(LogShaders, Error, "Archiving failed: shader %ls (Len: %0.8x, CRC: %0.8x, SHA: %ls) was not compiled for archiving (Compile Flags: %0.8x)!", *ShaderName, Header.SourceLen, Header.SourceCRC, *Hash.ToString(), (uint32)Header.CompileFlags);
			}
		}
		else
		{
			UE_LOGF(LogShaders, Error, "Archiving failed: shader SHA: %ls was not compiled to bytecode (%d)!", *Hash.ToString(), OfflineCompiledFlag);
		}
	}
	else
	{
		UE_LOGF(LogShaders, Error, "Archiving failed: no Xcode install on the local machine or a remote Mac.");
	}
	return Id;
}

bool FinalizeLibrary_Metal(FName const& Format, FString const& WorkingDir, FString const& LibraryPath, TArrayView<const uint64> Shaders, class FString const& DebugOutputDir, bool bStripLibs)
{
	bool bOK = false;

	FString FullyQualifiedWorkingDir = FPaths::ConvertRelativePathToFull(WorkingDir);

	const FMetalCompilerToolchain* Toolchain = FMetalCompilerToolchain::Get();
	const bool bCompilerAvailable = Toolchain->IsCompilerAvailable();
	EShaderPlatform Platform = FMetalCompilerToolchain::MetalShaderFormatToLegacyShaderPlatform(Format);
	const EAppleSDKType SDK = FMetalCompilerToolchain::MetalShaderPlatformToSDK(Platform);
    bool bCompiledWithMetalShaderConverter = FDataDrivenShaderPlatformInfo::GetSupportsBindless(Platform) &&
											 FShaderPlatformConfig::GetBindlessConfiguration(Platform) != ERHIBindlessConfiguration::Disabled;
    
	if (bCompilerAvailable)
	{
		int32 ReturnCode = 0;
		FString Results;
		FString Errors;
		
		// WARNING: This may be called from multiple threads using the same WorkingDir. All the temporary files must be uniquely named.
		// The local path that will end up with the archive.
		FString LocalArchivePath = FPaths::CreateTempFilename(*FullyQualifiedWorkingDir, TEXT("MetalArchive"), TEXT("")) + TEXT(".metalar");
		IFileManager::Get().Delete(*LocalArchivePath);
		IFileManager::Get().Delete(*LibraryPath);

		UE_LOGF(LogMetalShaderCompiler, Display, "Creating Native Library %ls", *LibraryPath);
        
		bool bArchiveFileValid = false;

        if (bCompiledWithMetalShaderConverter)
        {
            // Merge .metallib into a single .metallib.
            
            // Number of air per batch (limited by PlatformProcessLimits::MaxArgvParameters).
            static constexpr int32 NumArgcPerBatch = 96;
    
            TArray<FString> MetalPackArgsBatches;
            FString MetalPackArgs;
            uint32 CurPackArgsArgc = 0;
    
            uint32 Index = 0;
            for (auto Shader : Shaders)
            {
                uint32 Len = (Shader >> 32);
                uint32 CRC = (Shader & 0xffffffff);

                FString FileName = FString::Printf(TEXT("Main_%0.8x_%0.8x.o"), Len, CRC);

                UE_LOGF(LogMetalShaderCompiler, Verbose, "[%d/%d] %ls %ls", ++Index, Shaders.Num(), *Format.GetPlainNameString(), *FileName);

                FString SourceFilePath = FString::Printf(TEXT("\"%s/%s\""), *FullyQualifiedWorkingDir, *FileName);

				MetalPackArgs += FString::Printf(TEXT("%s "), *SourceFilePath);
                CurPackArgsArgc++;

                if (CurPackArgsArgc > NumArgcPerBatch)
                {
					MetalPackArgsBatches.Add(MetalPackArgs);
					MetalPackArgs.Empty(); // TODO: Might switch to SetNum(0); as Empty() implicitly reallocs (IIRC)
                    CurPackArgsArgc = 0;
                }
            }

            // Add pending batch to the list.
            if (MetalPackArgs.Len() > 0)
            {
				MetalPackArgsBatches.Add(MetalPackArgs);
            }
            bArchiveFileValid = (Shaders.Num() > 0);

            {
                // handle compile error
                if (ReturnCode != 0 || !bArchiveFileValid)
				{
					UE_LOGF(LogShaders, Error, "Archiving failed: no valid input for metal-pack.");
					return false;
				}
				
				// MetalPack each batch.
				FString BatchMergeArgs;
				for (int32 BatchIdx = 0; BatchIdx < MetalPackArgsBatches.Num(); BatchIdx++)
				{
					FString BatchOutputFile = FString::Printf(TEXT("MetalPackBatch_%d_"), BatchIdx);
					FString BatchOutputPath = FPaths::CreateTempFilename(*FullyQualifiedWorkingDir, *BatchOutputFile);

					BatchMergeArgs += FString::Printf(TEXT("\"%s\" "), *BatchOutputPath);

                    UE_LOGF(LogMetalShaderCompiler, Display, "[%d/%d] %ls", (BatchIdx + 1), MetalPackArgsBatches.Num(), *BatchOutputPath);

					FString MetalPackParams = FString::Printf(TEXT("-pack-metallibs internal -pack-descriptors internal -pack-reflections internal %s -o \"%s\""), *MetalPackArgsBatches[BatchIdx], *BatchOutputPath);
					ReturnCode = 0;
					Results.Empty();
					Errors.Empty();

					bool bSuccess = Toolchain->ExecMetalPack(SDK, *MetalPackParams, &ReturnCode, &Results, &Errors);

					// handle compile error
					if (!bSuccess || ReturnCode != 0)
					{
						UE_LOGF(LogShaders, Error, "Archiving failed: metal-pack failed with code %d: %ls %ls", ReturnCode, *Results, *Errors);
					}
					
					if (bStripLibs)
					{
						bSuccess = Toolchain->StripMetalLib(SDK, BatchOutputPath, BatchOutputPath, false);
					}
				}

				// Final pass: merge all batches into a single lib.
				UE_LOGF(LogMetalShaderCompiler, Display, "Post-processing archive for shader platform: %ls", *Format.GetPlainNameString());

				FString LocalMetalLibPath = LibraryPath;

				if (FPaths::FileExists(LocalMetalLibPath))
				{
					UE_LOGF(LogMetalShaderCompiler, Warning, "Archiving warning: target metallib already exists and will be overwritten: %ls", *LocalMetalLibPath);
				}

				FString MetalPackParams = FString::Printf(TEXT("-pack-metallibs internal -pack-descriptors internal -pack-reflections internal %s -o \"%s\""), *BatchMergeArgs, *LocalMetalLibPath);
				ReturnCode = 0;
				Results.Empty();
				Errors.Empty();

				bool bSuccess = Toolchain->ExecMetalPack(SDK, *MetalPackParams, &ReturnCode, &Results, &Errors);

				// handle compile error
				if (bSuccess && ReturnCode == 0)
				{
					check(LocalMetalLibPath == LibraryPath);
					
					bOK = (IFileManager::Get().FileSize(*LibraryPath) > 0);
					
					if (!bOK)
					{
						UE_LOGF(LogShaders, Error, "Archiving failed: failed to copy to local destination: %ls", *LibraryPath);
						return bOK;
					}
				}
				else
				{
					UE_LOGF(LogShaders, Error, "Archiving failed: metal-pack failed with code %d: %ls %ls", ReturnCode, *Results, *Errors);
					return bOK;
				}
            }
        }
        else
		{
			// Archive build phase - like unix ar, build metal archive from all the object files
			UE_LOGF(LogMetalShaderCompiler, Display, "Archiving %d shaders for shader platform: %ls", Shaders.Num(), *Format.GetPlainNameString());

			/*
				AR type utils can use 'M' scripts
				They look like this:
				CREATE MyArchive.metalar
				ADDMOD Shader1.metal
				ADDMOD Shader2.metal
				SAVE
				END
			*/
			FString M_Script = FString::Printf(TEXT("CREATE \"%s\"\n"), *FPaths::ConvertRelativePathToFull(LocalArchivePath));

			uint32 Index = 0;
			for (auto Shader : Shaders)
			{
				uint32 Len = (Shader >> 32);
				uint32 CRC = (Shader & 0xffffffff);

				FString FileName = FString::Printf(TEXT("Main_%0.8x_%0.8x.o"), Len, CRC);

				UE_LOGF(LogMetalShaderCompiler, Verbose, "[%d/%d] %ls %ls", ++Index, Shaders.Num(), *Format.GetPlainNameString(), *FileName);
				FString SourceFilePath = FString::Printf(TEXT("\"%s/%s\""), *FullyQualifiedWorkingDir, *FileName);

				M_Script += FString::Printf(TEXT("ADDMOD %s\n"), *SourceFilePath);
			}

			M_Script += FString(TEXT("SAVE\n"));
			M_Script += FString(TEXT("END\n"));

			FString LocalScriptFilePath = FPaths::CreateTempFilename(*FullyQualifiedWorkingDir, TEXT("MetalArScript"), TEXT(".M"));

			FFileHelper::SaveStringToFile(M_Script, *LocalScriptFilePath);

			if (!FPaths::FileExists(LocalScriptFilePath))
			{
				UE_LOGF(LogMetalShaderCompiler, Error, "Failed to create metal-ar .M script at %ls", *LocalScriptFilePath);
				return false;
			}

			FPaths::MakePlatformFilename(LocalScriptFilePath);
			bool bSuccess = Toolchain->ExecMetalAr(SDK, *LocalScriptFilePath, &ReturnCode, &Results, &Errors);
			bArchiveFileValid = FPaths::FileExists(LocalArchivePath);
					
			if (ReturnCode != 0 || !bArchiveFileValid)
			{
				UE_LOGF(LogMetalShaderCompiler, Error, "Archiving failed: metal-ar failed with code %d: %ls %ls", ReturnCode, *Results, *Errors);
				return false;
			}
		}
		
		// Lib build phase, metalar to metallib 
        if (!bCompiledWithMetalShaderConverter)
		{
			// handle compile error
			if (ReturnCode != 0 || !bArchiveFileValid)
			{
				UE_LOGF(LogShaders, Error, "Archiving failed: no valid input for metallib.");
				return false;
			}
			
			UE_LOGF(LogMetalShaderCompiler, Display, "Post-processing archive for shader platform: %ls", *Format.GetPlainNameString());

			FString LocalMetalLibPath = LibraryPath;

			if (FPaths::FileExists(LocalMetalLibPath))
			{
				UE_LOGF(LogMetalShaderCompiler, Warning, "Archiving warning: target metallib already exists and will be overwritten: %ls", *LocalMetalLibPath);
			}
		
			FString MetallibParams = FString::Printf(TEXT("-o \"%s\" \"%s\""), *LocalMetalLibPath, *LocalArchivePath);
			ReturnCode = 0;
			Results.Empty();
			Errors.Empty();
			
			bool bSuccess = Toolchain->ExecMetalLib(SDK, *MetallibParams, &ReturnCode, &Results, &Errors);

			// handle compile error
			if (!bSuccess || ReturnCode != 0)
			{
				UE_LOGF(LogShaders, Error, "Archiving failed: metallib failed with code %d: %ls %ls", ReturnCode, *Results, *Errors);
				return false;
			}
			
			check(LocalMetalLibPath == LibraryPath);

			bOK = (IFileManager::Get().FileSize(*LibraryPath) > 0);

			if (!bOK)
			{
				UE_LOGF(LogShaders, Error, "Archiving failed: failed to copy to local destination: %ls", *LibraryPath);
				return false;
			}
			
			if (bStripLibs)
			{
				bOK = Toolchain->StripMetalLib(SDK, LibraryPath, LibraryPath, true);
			}
		}
	}
	else
	{
		UE_LOGF(LogShaders, Error, "Archiving failed: no Xcode install.");
	}
	
	return bOK;
}

static void ReplaceString(FAnsiString& Str, int32 Pos, int Len, const FAnsiString& NewStr)
{
	Str.RemoveAt(Pos, Len, EAllowShrinking::No);
	Str.InsertAt(Pos, NewStr);
}

// Replace the special texture "gl_LastFragData" to a native subpass fetch operation. Returns true if the input source has been modified.
bool PatchSpecialTextureInHlslSource(FAnsiString& SourceData, uint32* OutSubpassInputsDim, uint32 SubpassInputDimCount)
{
	bool bSourceDataWasModified = false;

	// Invalidate output parameter for dimension of subpass input attachemnt at slot 0 (primary slot for "gl_LastFragData").
	FMemory::Memzero(OutSubpassInputsDim, sizeof(uint32) * SubpassInputDimCount);
	
	// Check if special texture is present in the code
	const char* GSpecialTextureLastFragData = "gl_LastFragData";
	if (SourceData.Find(GSpecialTextureLastFragData, ESearchCase::CaseSensitive) != INDEX_NONE)
	{
		struct FHlslVectorType
		{
			const char* TypenameIdent;
			const char* TypenameSuffix;
			uint32 Dimension;
		};
		const FHlslVectorType FragDeclTypes[4] =
		{
			{ "float4", "RGBA", 4 },
			{ "float",	"R",	1 },
			{ "half4",  "RGBA", 4 },
			{ "half",   "R",    1 }
		};
		
		// Replace declaration of special texture with corresponding 'SubpassInput' declaration with respective dimension, i.e. float, float4, etc.
		for (uint32 SubpassIndex = 0; SubpassIndex < SubpassInputDimCount; SubpassIndex++)
		{
			for (const FHlslVectorType& FragDeclType : FragDeclTypes)
			{
				// Try to find "Texture2D<T>" or "Texture2D< T >" (where T is the vector type), because a rewritten HLSL might have changed the formatting.
				FAnsiString LastFragDataN = FAnsiString::Printf("%s%s_%u", GSpecialTextureLastFragData, FragDeclType.TypenameSuffix, SubpassIndex);
				FAnsiString FragDecl = FAnsiString::Printf("Texture2D<%s> %s;", FragDeclType.TypenameIdent, *LastFragDataN);
				int32 FragDeclIncludePos = SourceData.Find(FragDecl, ESearchCase::CaseSensitive);
			
				if (FragDeclIncludePos == INDEX_NONE)
				{
					FragDecl = FAnsiString::Printf("Texture2D< %s > %s;", FragDeclType.TypenameIdent, *LastFragDataN);
					FragDeclIncludePos = SourceData.Find(FragDecl, ESearchCase::CaseSensitive);
				}
			
				if (FragDeclIncludePos != INDEX_NONE)
				{
					// Replace declaration of Texture2D<T> with SubpassInput<T>
					ReplaceString(SourceData,
						FragDeclIncludePos,
						FragDecl.Len(),
						FAnsiString::Printf("[[vk::input_attachment_index(%d)]] SubpassInput<%s> %s;", SubpassIndex, FragDeclType.TypenameIdent, *LastFragDataN));

					OutSubpassInputsDim[SubpassIndex] = FragDeclType.Dimension;

					// Replace all uses of special texture by 'SubpassLoad' operation
					FAnsiString FragLoad = FAnsiString::Printf("%s.Load(uint3(0, 0, 0), 0)", *LastFragDataN);
					for (int32 FragLoadIncludePos = 0; (FragLoadIncludePos = SourceData.Find(FragLoad, ESearchCase::CaseSensitive, ESearchDir::FromStart, FragLoadIncludePos)) != INDEX_NONE;)
					{
						ReplaceString(SourceData,
							FragLoadIncludePos,
							FragLoad.Len(),
							FAnsiString::Printf("%s.SubpassLoad()", *LastFragDataN));
					}

					// Mark source data as being modified
					bSourceDataWasModified = true;
					
					break;
				}
			}
		}
	}
	
	return bSourceDataWasModified;
}
