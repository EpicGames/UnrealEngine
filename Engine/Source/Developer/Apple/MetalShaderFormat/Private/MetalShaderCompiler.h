// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Misc/SecureHash.h"
#include "ShaderCompilerCommon.h"
#include "MetalBackend.h"

#include <string>

struct FMetalShaderBytecodeJob
{
	FName ShaderFormat;
    FString Defines;
	FString TmpFolder;
	FString InputFile;
	FString InputPCHFile;
	FString OutputFile;
	FString OutputObjectFile;
	FString CompilerVersion;
	FString MinOSVersion;
	FString PreserveInvariance;
	FString DebugInfo;
	FString MathMode;
	FString Standard;
	FString IncludeDir;
	FString ModuleCacheDirectory;
	uint32 SourceCRCLen;
	uint32 SourceCRC;
	bool bUseNativeShaderLibrary;
	bool bCompileAsPCH;
	bool bOptimizeForSize;
	bool bStripIndividualMetalLibs;
	
	FString Message;
	FString Results;
	FString Errors;
	int32 ReturnCode;
};

struct FMetalShaderPreprocessed
{
	FString NativePath;
	TArray<uint8> OutputFile;
	TArray<uint8> ObjectFile;
	
	friend FArchive& operator<<( FArchive& Ar, FMetalShaderPreprocessed& Info )
	{
		Ar << Info.NativePath << Info.OutputFile << Info.ObjectFile;
		return Ar;
	}
};

struct FMetalShaderOutputMetaData
{
	uint32 InvariantBuffers = 0;
	uint32 TypedBuffers = 0;
	uint32 TypedUAVs = 0;
	uint32 ConstantBuffers = 0;
};

// Replace the special texture "gl_LastFragData" to a native subpass fetch operation. Returns true if the input source has been modified.
bool PatchSpecialTextureInHlslSource(FAnsiString& SourceData, uint32* OutSubpassInputsDim, uint32 SubpassInputDimCount);
