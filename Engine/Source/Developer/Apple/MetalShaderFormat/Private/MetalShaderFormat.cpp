// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetalShaderFormat.h"
#include "DXCWrapper.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/IShaderFormat.h"
#include "Interfaces/IShaderFormatModule.h"
#include "ShaderCore.h"
#include "ShaderCodeArchive.h"
#include "ShaderPreprocessTypes.h"
#include "MetalShaderResources.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Serialization/Archive.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "MetalBackend.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "FileUtilities/ZipArchiveWriter.h"
#include "MetalShaderCompiler.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "ShaderPlatformConfig.h"
#include "RHIStrings.h"
#include "Containers/ArrayView.h"
#include "Templates/Sorting.h"
#include "ir_version.h"

DEFINE_LOG_CATEGORY(LogMetalCompilerSetup)
DEFINE_LOG_CATEGORY(LogMetalShaderCompiler)

#define WRITE_METAL_SHADER_SOURCE_ARCHIVE 0

// Set this cvar to get additional logging information about Metal toolchain setup.
static int32 GCheckCompilerToolChainSetup = 0;
static FAutoConsoleVariableRef CVarCheckCompilerToolChainSetup(
	TEXT("Metal.CheckCompilerToolChainSetup"),
	GCheckCompilerToolChainSetup,
	TEXT("Should we check the Metal Compiler ToolChain Setup."),
	ECVF_Default
);

static TAutoConsoleVariable<int32> CVarNumShadersPerLibrary(
	TEXT("Metal.NumShadersPerLibrary"), 10000,
	TEXT("Set the maximum number of shaders per library."),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarNumShadersPerLibraryBindlessWithSymbols(
	TEXT("Metal.NumShadersPerLibraryBindlessWithSymbols"), 1000,
	TEXT("Set the maximum number of shaders per bindless library when symbols are enabled."),
	ECVF_Default);

extern bool PreprocessMetalShader(const FShaderCompilerInput& Input, const FShaderCompilerEnvironment& Environment, FShaderPreprocessOutput& PreprocessOutput);
extern void CompileMetalShader(const FShaderCompilerInput& Input, const FShaderPreprocessOutput& InPreprocessOutput, FShaderCompilerOutput& Output);
extern void OutputMetalDebugData(const FShaderCompilerInput& Input, const FShaderPreprocessOutput& PreprocessOutput, const FShaderCompilerOutput& Output);

extern bool StripShader_Metal(TArray<uint8>& Code, class FString const& DebugPath, bool const bNative);
extern uint64 AppendShader_Metal(class FString const& ArchivePath, const FShaderHash& Hash, TArray<uint8>& Code);
extern bool FinalizeLibrary_Metal(class FName const& Format, class FString const& ArchivePath, class FString const& LibraryPath, TArrayView<const uint64> Shaders, class FString const& DebugOutputDir, bool bStripLibs);

/** Version for shader format, this becomes part of the DDC key. */
static const FGuid UE_SHADER_METAL_VER = FGuid("B0DC25EF-C34D-437A-94E5-5E5146AF1B9A");

class FMetalShaderFormat : public UE::ShaderCompilerCommon::FBaseShaderFormat 
{
	const uint32 ShaderConductorVersionHash;

public:
	FMetalShaderFormat(uint32 InShaderConductorVersionHash)
		: ShaderConductorVersionHash(InShaderConductorVersionHash)
	{
		FMetalCompilerToolchain::CreateAndInit();
	}
	virtual ~FMetalShaderFormat()
	{
		FMetalCompilerToolchain::Destroy();
	}
	virtual uint32 GetVersion(FName Format) const override final
	{
		// If there's no compiler on this machine, this is irrelevant so just return 0
		if (!FMetalCompilerToolchain::Get()->IsCompilerAvailable())
		{
			return 0;
		}

		bool bUseFullMetalVersion = false;
		EShaderPlatform ShaderPlatform = FMetalCompilerToolchain::MetalShaderFormatToLegacyShaderPlatform(Format);

		if (FMetalCompilerToolchain::IsMobile(ShaderPlatform))
		{
			GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("UseFullMetalVersionInShaderVersion"), bUseFullMetalVersion, GEngineIni);
		}
		else
		{
			GConfig->GetBool(TEXT("/Script/MacTargetPlatform.MacTargetSettings"), TEXT("UseFullMetalVersionInShaderVersion"), bUseFullMetalVersion, GEngineIni);
		}

		FMetalCompilerToolchain::PackedVersion MetalVersionNumber = FMetalCompilerToolchain::Get()->GetCompilerVersion(ShaderPlatform);
		uint16 HashValue = MetalVersionNumber.Major;

		if (bUseFullMetalVersion)
		{
			// Use entire Metal version if .ini settings instruct us to do so (e.g. p4 dev build)
			HashValue ^= MetalVersionNumber.Minor;
			HashValue ^= MetalVersionNumber.Patch;
		}
		else
		{
			// Only use Metal major version (e.g. Installed build)
			// Since Metal minor/patch version changes every Xcode minor version, we don't want users to rebuild shaders for every minor version update
		}

		uint32 Result = GetTypeHash(HashValue);

		Result = HashCombine(Result, GetTypeHash(UE_SHADER_METAL_VER));
		Result = HashCombine(Result, GetTypeHash(ShaderConductorVersionHash));

		// Read bindless configuration directly from config using the shader format name
		// TODO: Replace this in 5.9+ to enable for all platforms
		if (ShaderPlatform == SP_METAL_SM6)
		{
			Result = HashCombine(Result, GetTypeHash(IR_VERSION_MAJOR));
			Result = HashCombine(Result, GetTypeHash(IR_VERSION_MINOR));
			Result = HashCombine(Result, GetTypeHash(IR_VERSION_PATCH));
		}

		return Result;
	}

	virtual void GetSupportedFormats(TArray<FName>& OutFormats) const override final
	{
		OutFormats.Add(NAME_SF_METAL_ES3_1_IOS);
		OutFormats.Add(NAME_SF_METAL_SM5_IOS);
		OutFormats.Add(NAME_SF_METAL_SM6_IOS);
		OutFormats.Add(NAME_SF_METAL_ES3_1_TVOS);
		OutFormats.Add(NAME_SF_METAL_SM5_TVOS);
		OutFormats.Add(NAME_SF_METAL_SM5);
		OutFormats.Add(NAME_SF_METAL_SM6);
		OutFormats.Add(NAME_SF_METAL_SIM);
		OutFormats.Add(NAME_SF_METAL_ES3_1);
	}

	void CheckShaderFormat(FName Format) const
	{
		check(Format == NAME_SF_METAL_ES3_1_IOS
			|| Format == NAME_SF_METAL_SM5_IOS
			|| Format == NAME_SF_METAL_SM6_IOS
			|| Format == NAME_SF_METAL_ES3_1_TVOS
			|| Format == NAME_SF_METAL_SM5_TVOS
			|| Format == NAME_SF_METAL_SM5
			|| Format == NAME_SF_METAL_SM6
			|| Format == NAME_SF_METAL_SIM
			|| Format == NAME_SF_METAL_ES3_1);
	}

	virtual bool PreprocessShader(
		const FShaderCompilerInput& Input,
		const FShaderCompilerEnvironment& Environment,
		FShaderPreprocessOutput& PreprocessOutput) const override final
	{
		CheckShaderFormat(Input.ShaderFormat);
		return PreprocessMetalShader(Input, Environment, PreprocessOutput);
	}

	virtual void CompilePreprocessedShader(
		const FShaderCompilerInput& Input,
		const FShaderPreprocessOutput& PreprocessOutput,
		FShaderCompilerOutput& Output) const override final
	{
		CheckShaderFormat(Input.ShaderFormat);
		CompileMetalShader(Input, PreprocessOutput, Output);
	}

	virtual bool CanStripShaderCode(bool const bNativeFormat) const override final
	{
		return CanCompileBinaryShaders() && bNativeFormat;
	}

	virtual bool StripShaderCode( TArray<uint8>& Code, FString const& DebugOutputDir, bool const bNative ) const override final
	{
		return StripShader_Metal(Code, DebugOutputDir, bNative);
    }

	virtual bool SupportsShaderArchives() const override 
	{ 
		return CanCompileBinaryShaders();
	}

	virtual bool CreateShaderArchive(
		FString const& LibraryName,
		FName ShaderFormatAndShaderPlatformName,
		const FString& WorkingDirectory,
		const FString& OutputDir,
		const FString& DebugOutputDir,
		const FSerializedShaderArchive& InSerializedShaders,
		const TArray<FSharedBuffer>& ShaderCode,
		TArray<FString>* OutputFiles) const override final
	{
		int32 NumShadersPerLibrary = CVarNumShadersPerLibrary.GetValueOnAnyThread();

		check(LibraryName.Len() > 0);

		TArray<FString> Components;
		FString ShaderPlatform = ShaderFormatAndShaderPlatformName.ToString();
		ShaderPlatform.ParseIntoArray(Components, TEXT("-"));
		check(Components.Num() == 2);
		FName ShaderFormatName(Components[0]);

		check(ShaderFormatName == NAME_SF_METAL_ES3_1_IOS ||
			  ShaderFormatName == NAME_SF_METAL_SM5_IOS ||
			  ShaderFormatName == NAME_SF_METAL_SM6_IOS ||
			  ShaderFormatName == NAME_SF_METAL_ES3_1_TVOS ||
			  ShaderFormatName == NAME_SF_METAL_SM5_TVOS ||
			  ShaderFormatName == NAME_SF_METAL_SM5 ||
			  ShaderFormatName == NAME_SF_METAL_SM6 ||
			  ShaderFormatName == NAME_SF_METAL_SIM ||
			  ShaderFormatName == NAME_SF_METAL_ES3_1);

		// Bindless needs a lower limit of shaders per library as the packing process takes a significant amount of RAM when symbols are enabled
		const EShaderPlatform Platform = FMetalCompilerToolchain::MetalShaderFormatToLegacyShaderPlatform(ShaderFormatName);
		if(FDataDrivenShaderPlatformInfo::GetSupportsBindless(Platform)
		   && FShaderPlatformConfig::GetBindlessConfiguration(Platform) != ERHIBindlessConfiguration::Disabled
		   && ShouldGenerateShaderSymbols(ShaderFormatName))
		{
			NumShadersPerLibrary = CVarNumShadersPerLibraryBindlessWithSymbols.GetValueOnAnyThread();
		}
		
		const FString ArchivePath = (WorkingDirectory / ShaderFormatAndShaderPlatformName.ToString());
		IFileManager::Get().DeleteDirectory(*ArchivePath, false, true);
		IFileManager::Get().MakeDirectory(*ArchivePath);

		FSerializedShaderArchive SerializedShaders(InSerializedShaders);
		check(SerializedShaders.GetNumShaders() == ShaderCode.Num());

		TArray<uint8> StrippedShaderCode;
		TArray<uint8> TempShaderCode;

		TArray<uint64> AllShaders;
		AllShaders.SetNumUninitialized(SerializedShaders.GetNumShaders());

		for (int32 ShaderIndex = 0; ShaderIndex < SerializedShaders.GetNumShaders(); ++ShaderIndex)
		{
			SerializedShaders.DecompressShader(ShaderIndex, ShaderCode, TempShaderCode);
			StripShader_Metal(TempShaderCode, DebugOutputDir, true);

			uint64 ShaderId = AppendShader_Metal(ArchivePath, SerializedShaders.ShaderHashes[ShaderIndex], TempShaderCode);
			AllShaders[ShaderIndex] = ShaderId;

			FShaderCodeEntry& ShaderEntry = SerializedShaders.ShaderEntries[ShaderIndex];
			ShaderEntry.Size = TempShaderCode.Num();

			StrippedShaderCode.Append(TempShaderCode);
		}
		// SerializedShaders archive is finalized later, when we sort out
		// the duplicated entries.

		// This is used later to lookup the shader code entry by the
		// original shader index.
		TArray<uint64> UnsortedAllShaders(AllShaders);

		// Gives us stable ordering, while also clumping all
		// duplicates together, if any.
		RadixSort64(AllShaders.GetData(), AllShaders.Num());

		//
		// As we throw away duplicates, we need to re-calculate
		// each shader index in the library. That index is serialized
		// per shader code entry and is used during runtime loading.
		//
		// The loop below does both operations.
		//

		TMap<uint64, uint32> ShaderIdToIndexInLibrary;
		ShaderIdToIndexInLibrary.Reserve(AllShaders.Num());

		TArray<uint64> UniqueShaders;
		UniqueShaders.Reserve(AllShaders.Num());
		{
			uint64 LastShader = 0;
			for (uint64 Shader : AllShaders)
			{
				if (Shader == LastShader)
				{
					continue;
				}
				LastShader = Shader;

				ShaderIdToIndexInLibrary.Add(Shader, UniqueShaders.Num());
				UniqueShaders.Add(Shader);
			}

			UE_LOGF(LogShaders, Display, "Metallib %ls has %d shaders (%d unique)", *LibraryName, AllShaders.Num(), UniqueShaders.Num());
			AllShaders.Empty();
		}

		for (int32 ShaderIndex = 0; ShaderIndex < SerializedShaders.GetNumShaders(); ++ShaderIndex)
		{
			uint64 ShaderId = UnsortedAllShaders[ShaderIndex];
			FShaderCodeEntry& ShaderEntry = SerializedShaders.ShaderEntries[ShaderIndex];
			ShaderEntry.SetLibraryIndex(ShaderIdToIndexInLibrary[ShaderId]);
		}
		SerializedShaders.Finalize();

		TArrayView<const uint64> Workset = MakeArrayView(UniqueShaders.GetData(), UniqueShaders.Num());
		TArray<TArrayView<const uint64>> SubLibraries;
		
		int32 CurrentSubLibrarySize = 0;
		for (uint64 Shader : UniqueShaders)
		{
			++CurrentSubLibrarySize;
			if (CurrentSubLibrarySize == NumShadersPerLibrary)
			{
				SubLibraries.Add(Workset.Left(CurrentSubLibrarySize));
				Workset.RightChopInline(CurrentSubLibrarySize);
				CurrentSubLibrarySize = 0;
			}
		}
		if (!Workset.IsEmpty())
		{
			SubLibraries.Add(Workset);
		}

		bool bOK = false;
		FString LibraryPlatformName = FString::Printf(TEXT("%s_%s"), *LibraryName, *ShaderFormatAndShaderPlatformName.ToString());

		LibraryPlatformName.ToLowerInline();
		volatile int32 CompiledLibraries = 0;
		FGraphEventArray Tasks;

		for (int32 Index = 0; Index < SubLibraries.Num(); ++Index)
		{
			const TArrayView<const uint64>& PartialShaders = SubLibraries[Index];

			FString LibraryPath = (OutputDir / LibraryPlatformName) + FString::Printf(TEXT(".%d"), Index) + FMetalCompilerToolchain::MetalLibraryExtension;
			if (OutputFiles)
			{
				OutputFiles->Add(LibraryPath);
			}

			// Enqueue the library compilation as a task so we can go wide
			FGraphEventRef CompletionFence = FFunctionGraphTask::CreateAndDispatchWhenReady([ShaderFormatName, ArchivePath, LibraryPath, PartialShaders, DebugOutputDir, &CompiledLibraries, Platform]()
				{
					static const auto CVarSymbols = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Shaders.Symbols"));
					static const auto CVarExtraData = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Shaders.ExtraData"));
				
					bool bStripLibrary = (FDataDrivenShaderPlatformInfo::GetIsMobile(Platform) || FDataDrivenShaderPlatformInfo::GetSupportsBindless(Platform)) &&
											CVarSymbols->GetInt() == 0 &&
											CVarExtraData->GetInt() == 0;
				
					if (FinalizeLibrary_Metal(ShaderFormatName, ArchivePath, LibraryPath, PartialShaders, DebugOutputDir, bStripLibrary))
					{
						FPlatformAtomics::InterlockedIncrement(&CompiledLibraries);
					}
				}, TStatId(), NULL, ENamedThreads::AnyThread);

			Tasks.Add(CompletionFence);
		}

#if WITH_ENGINE
		FGraphEventRef DebugDataCompletionFence = FFunctionGraphTask::CreateAndDispatchWhenReady([ShaderFormatAndShaderPlatformName, OutputDir, LibraryPlatformName, DebugOutputDir]()
			{
				//TODO add a check in here - this will only work if we have shader archiving with debug info set.

				//We want to archive all the metal shader source files so that they can be unarchived into a debug location
				//This allows the debugging of optimised metal shaders within the xcode tool set
				//Currently using the 'tar' system tool to create a compressed tape archive

				//Place the archive in the same position as the .metallib file
				FString CompressedDir = (OutputDir / TEXT("../MetaData/ShaderDebug/"));
				IFileManager::Get().MakeDirectory(*CompressedDir, true);

				FString CompressedPath = (CompressedDir / LibraryPlatformName) + TEXT(".zip");

				IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
				IFileHandle* ZipFile = PlatformFile.OpenWrite(*CompressedPath);
				if (ZipFile)
				{
					FZipArchiveWriter* ZipWriter = new FZipArchiveWriter(ZipFile);

					//Find the metal source files
					TArray<FString> FilesToArchive;
					IFileManager::Get().FindFilesRecursive(FilesToArchive, *DebugOutputDir, TEXT("*.metal"), true, false, false);

					//Write the local file names into the target file
					const FString DebugDir = DebugOutputDir / *ShaderFormatAndShaderPlatformName.ToString();

					for (FString FileName : FilesToArchive)
					{
						TArray<uint8> FileData;
						FFileHelper::LoadFileToArray(FileData, *FileName);
						FPaths::MakePathRelativeTo(FileName, *DebugDir);

						ZipWriter->AddFile(FileName, FileData, FDateTime::Now());
					}

					delete ZipWriter;
					ZipWriter = nullptr;
				}
				else
				{
					UE_LOGF(LogShaders, Error, "Failed to create Metal debug .zip output file \"%ls\". Debug .zip export will be disabled.", *CompressedPath);
				}
			}, TStatId(), NULL, ENamedThreads::AnyThread);
		Tasks.Add(DebugDataCompletionFence);
#endif // WITH_ENGINE

		FTaskGraphInterface::Get().WaitUntilTasksComplete(Tasks);

		if (CompiledLibraries == SubLibraries.Num())
		{
			FString BinaryShaderFile = (OutputDir / LibraryPlatformName) + FMetalCompilerToolchain::MetalMapExtension;
			BinaryShaderFile.ToLowerInline();
			FArchive* BinaryShaderAr = IFileManager::Get().CreateFileWriter(*BinaryShaderFile);
			if (BinaryShaderAr != NULL)
			{
				FMetalShaderLibraryHeader Header;
				Header.Format = ShaderFormatName.GetPlainNameString();
				Header.NumLibraries = SubLibraries.Num();
				Header.NumShadersPerLibrary = NumShadersPerLibrary;

				*BinaryShaderAr << Header;
				*BinaryShaderAr << SerializedShaders;
				*BinaryShaderAr << StrippedShaderCode;

				BinaryShaderAr->Flush();
				delete BinaryShaderAr;

				if (OutputFiles)
				{
					OutputFiles->Add(BinaryShaderFile);
				}

				bOK = true;
			}
		}

		return bOK;

		//Map.Format = Format.GetPlainNameString();
	}

	virtual void ModifyShaderCompilerInput(FShaderCompilerInput& Input) const override
	{
		// Work out which standard we need, this is dependent on the shader platform.
		const bool bIsMobile = FMetalCompilerToolchain::Get()->IsMobile((EShaderPlatform)Input.Target.Platform);
		if (bIsMobile)
		{
			Input.Environment.SetDefine(TEXT("IOS"), 1);
		}
		else
		{
			Input.Environment.SetDefine(TEXT("MAC"), 1);
		}

		Input.Environment.SetDefine(TEXT("COMPILER_METAL"), 1);

		if (Input.ShaderFormat == NAME_SF_METAL_ES3_1_IOS ||
			Input.ShaderFormat == NAME_SF_METAL_ES3_1_TVOS ||
			Input.ShaderFormat == NAME_SF_METAL_SIM ||
			Input.ShaderFormat == NAME_SF_METAL_ES3_1)
		{
			Input.Environment.SetDefine(TEXT("METAL_ES3_1_PROFILE"), 1);
		}
		else if (Input.ShaderFormat == NAME_SF_METAL_SM5_IOS || Input.ShaderFormat == NAME_SF_METAL_SM5_TVOS)
		{
			Input.Environment.SetDefine(TEXT("METAL_SM5_IOS_TVOS_PROFILE"), 1);
		}
		else if (Input.ShaderFormat == NAME_SF_METAL_SM5)
		{
			Input.Environment.SetDefine(TEXT("METAL_SM5_PROFILE"), 1);
		}
		else if (Input.ShaderFormat == NAME_SF_METAL_SM6 || Input.ShaderFormat == NAME_SF_METAL_SM6_IOS)
		{
			Input.Environment.SetDefine(TEXT("METAL_SM6_PROFILE"), 1);
			Input.Environment.SetDefine(TEXT("PLATFORM_SUPPORTS_CALLABLE_SHADERS"), true);
		}

		Input.Environment.SetDefine(TEXT("COMPILER_HLSLCC"), 2);
		const bool bUseMetalShaderConverter = FDataDrivenShaderPlatformInfo::GetSupportsBindless((EShaderPlatform)Input.Target.Platform)
												&& FShaderPlatformConfig::GetBindlessConfiguration((EShaderPlatform)Input.Target.Platform) != ERHIBindlessConfiguration::Disabled;
		
		if (bUseMetalShaderConverter)
		{
			Input.Environment.SetDefine(TEXT("PLATFORM_SUPPORTS_STATIC_SAMPLERS"), 1);
			Input.Environment.SetDefine(TEXT("COMPILER_METAL_SHADER_CONVERTER"), 1);
		}
		
		if (!bUseMetalShaderConverter)
		{
			if (Input.Environment.FullPrecisionInPS || (IsValidRef(Input.SharedEnvironment) && Input.SharedEnvironment->FullPrecisionInPS))
			{
				Input.Environment.SetDefine(TEXT("FORCE_FLOATS"), (uint32)1);
			}
		}
		else
		{
			// We can use 16bits types with Msc (since we do not use the frontend).
			if (Input.Environment.CompilerFlags.Contains(CFLAG_AllowRealTypes))
			{
				Input.Environment.SetDefine(TEXT("PLATFORM_SUPPORTS_REAL_TYPES"), 1);
			}
		}
		
		if (Input.Environment.CompilerFlags.Contains(CFLAG_AvoidFlowControl)
			|| Input.Environment.CompilerFlags.Contains(CFLAG_PreferFlowControl))
		{
			Input.Environment.SetDefine(TEXT("COMPILER_SUPPORTS_ATTRIBUTES"), (uint32)0);
		}
		else
		{
			Input.Environment.SetDefine(TEXT("COMPILER_SUPPORTS_ATTRIBUTES"), (uint32)1);
		}

		bool bUsesInlineRayTracing = Input.Environment.CompilerFlags.Contains(CFLAG_InlineRayTracing);
		if (bUsesInlineRayTracing)
		{
			Input.Environment.SetDefine(TEXT("PLATFORM_SUPPORTS_INLINE_RAY_TRACING"), 1);
		}

		Input.Environment.SetDefine(TEXT("COMPILER_SUPPORTS_DUAL_SOURCE_BLENDING_SLOT_DECORATION"), (uint32)1);
		Input.Environment.SetDefine(TEXT("PLATFORM_SUPPORTS_CONSTANTBUFFER_OBJECT"), true);
	}
	
	virtual bool CanCompileBinaryShaders() const override final
	{
#if PLATFORM_MAC
		return FPlatformMisc::IsSupportedXcodeVersionInstalled();
#else
		return FMetalCompilerToolchain::Get()->IsCompilerAvailable();
#endif
	}
	virtual const TCHAR* GetPlatformIncludeDirectory() const
	{
		return TEXT("Metal");
	}
};

/**
 * Module for Metal shaders
 */

static IShaderFormat* Singleton = nullptr;

class FMetalShaderFormatModule : public IShaderFormatModule, public FShaderConductorModuleWrapper
{
public:
	virtual ~FMetalShaderFormatModule()
	{
		Singleton = nullptr;
	}

	virtual IShaderFormat* GetShaderFormat()
	{
		return Singleton;
	}

	virtual void StartupModule() override
	{
		Singleton = new FMetalShaderFormat(FShaderConductorModuleWrapper::GetModuleVersionHash());
	}

	virtual void ShutdownModule() override
	{
		if (Singleton != nullptr)
		{
			delete Singleton;
			Singleton = nullptr;
		}
	}
};

IMPLEMENT_MODULE(FMetalShaderFormatModule, MetalShaderFormat);

static FMetalCompilerToolchain::EMetalToolchainStatus ParseCompilerVersionAndTarget(const FString& OutputOfMetalDashV, FString& OutVersionString, FMetalCompilerToolchain::PackedVersion& OutPackedVersionNumber, FMetalCompilerToolchain::PackedVersion& OutPackedTargetNumber)
{
	/*
		Output of metal -v might look like this:
		Apple LLVM version 902.11 (metalfe-902.11.1)
		Target: air64-apple-darwin19.5.0
		Thread model: posix
		InstalledDir: C:\Program Files\Metal Developer Tools\ios\bin
	*/

	// Default initialize output parameters
	OutVersionString.Empty();
	OutPackedVersionNumber = {};
	OutPackedTargetNumber = {};

	TArray<FString> Lines;
	OutputOfMetalDashV.ParseIntoArrayLines(Lines);
	int32 VersionLineIndex = 0;

	for (int32 Index = 0; Index < Lines.Num(); ++Index)
	{
		if (Lines[Index].StartsWith(TEXT("Apple ")) && Lines[Index].Contains(TEXT(" version ")) && Lines[Index].EndsWith(TEXT(")")))
		{
			VersionLineIndex = Index;
			break;
		}
	}

	if (VersionLineIndex < Lines.Num())
	{
		OutVersionString = Lines[VersionLineIndex];
		FString& Version = Lines[VersionLineIndex];
		check(!Version.IsEmpty());

		int32 Major = 0, Minor = 0;
		int32 NumResults = 0;
#if !PLATFORM_WINDOWS
		char AppleToolName[PATH_MAX] = { '\0' };
		char SupplementaryVersionName[PATH_MAX] = { '\0' };
		NumResults = sscanf(TCHAR_TO_ANSI(*Version), "Apple %s version %d.%d (metalfe-%s)", AppleToolName, &Major, &Minor, SupplementaryVersionName);
#else
		TCHAR AppleToolName[WINDOWS_MAX_PATH] = { '\0' };
		TCHAR SupplementaryVersionName[WINDOWS_MAX_PATH] = { '\0' };
		NumResults = swscanf_s(*Version, TEXT("Apple %ls version %d.%d (metalfe-%ls)"), AppleToolName, WINDOWS_MAX_PATH, &Major, &Minor, SupplementaryVersionName, WINDOWS_MAX_PATH);
#endif
		if (NumResults != 4)
		{
			UE_LOGF(LogMetalCompilerSetup, Warning, "Metal version string format unrecoginzed");
			UE_LOGF(LogMetalCompilerSetup, Warning, "Expecting: Apple LLVM version 902.11 (metalfe-902.11.1)");
			UE_LOGF(LogMetalCompilerSetup, Warning, "Obtained: %ls", *Version);
		}
		
		OutPackedVersionNumber.Major = Major;
		OutPackedVersionNumber.Minor = Minor;
		// The version name in brackets is too irregular to extract a useful patch version
		// Sometimes (metalfe-31001.667.2), sometimes (metalfe-31001.643.2.1), sometimes (metalfe-31001.362-windows)
		OutPackedVersionNumber.Patch = 0;
	}

	if (OutPackedVersionNumber.Version == 0)
	{
		return FMetalCompilerToolchain::EMetalToolchainStatus::CouldNotParseCompilerVersion;
	}

	if (VersionLineIndex + 1 < Lines.Num())
	{
		const FString& FormatVersion = Lines[VersionLineIndex + 1];
		int32 Major = 0, Minor = 0, Patch = 0;
		int32 NumResults = 0;
#if !PLATFORM_WINDOWS
		NumResults = sscanf(TCHAR_TO_ANSI(*FormatVersion), "Target: air64-apple-darwin%d.%d.%d", &Major, &Minor, &Patch);
#else
		NumResults = swscanf_s(*FormatVersion, TEXT("Target: air64-apple-darwin%d.%d.%d"), &Major, &Minor, &Patch);
#endif
		OutPackedTargetNumber.Major = Major;
		OutPackedTargetNumber.Minor = Minor;
		OutPackedTargetNumber.Patch = Patch;
	}

	if (OutPackedTargetNumber.Version == 0)
	{
		return FMetalCompilerToolchain::EMetalToolchainStatus::CouldNotParseTargetVersion;
	}

	return FMetalCompilerToolchain::EMetalToolchainStatus::Success;
}

static FMetalCompilerToolchain::EMetalToolchainStatus ParseLibraryToolpath(const FString& OutputOfMetalSearchDirs, FString& LibraryPath)
{
	static FString LibraryPrefix(TEXT("libraries: =%s"));

	TArray<FString> Lines;
	OutputOfMetalSearchDirs.ParseIntoArrayLines(Lines);
	{
		int32 LibrariesLineIndex = 0;
		
		for (int32 Index = 0; Index < Lines.Num(); ++Index)
		{
			if (Lines[Index].StartsWith(TEXT("libraries: =")))
			{
				LibrariesLineIndex = Index;
				break;
			}
		}

		FString& LibraryLine = Lines[LibrariesLineIndex];

		LibraryPath = LibraryLine.RightChop(LibraryPrefix.Len());

		if (!FPaths::DirectoryExists(LibraryPath))
		{
			return FMetalCompilerToolchain::EMetalToolchainStatus::CouldNotFindMetalStdLib;
		}

		FPaths::Combine(LibraryPath, TEXT("include"), TEXT("metal"));

		if (!FPaths::DirectoryExists(LibraryPath))
		{
			return FMetalCompilerToolchain::EMetalToolchainStatus::CouldNotFindMetalStdLib;
		}
	}

	return FMetalCompilerToolchain::EMetalToolchainStatus::Success;
}

FMetalCompilerToolchain* FMetalCompilerToolchain::Singleton = nullptr;
FString FMetalCompilerToolchain::MetalExtention(TEXT(".metal"));
FString FMetalCompilerToolchain::MetalLibraryExtension(TEXT(".metallib"));
FString FMetalCompilerToolchain::MetalObjectExtension(TEXT(".air"));
#if PLATFORM_WINDOWS
FString FMetalCompilerToolchain::MetalFrontendBinary(TEXT("metal.exe"));
FString FMetalCompilerToolchain::MetalArBinary(TEXT("metal-ar.exe"));
FString FMetalCompilerToolchain::MetalLibraryBinary(TEXT("metallib.exe"));
FString FMetalCompilerToolchain::MetalPackBinary(TEXT("metal-pack.exe"));
FString FMetalCompilerToolchain::MetalStripBinary(TEXT("metal-strip.exe"));
#else
FString FMetalCompilerToolchain::MetalFrontendBinary(TEXT("metal"));
FString FMetalCompilerToolchain::MetalArBinary(TEXT("metal-ar"));
FString FMetalCompilerToolchain::MetalLibraryBinary(TEXT("metallib"));
FString FMetalCompilerToolchain::MetalPackBinary(TEXT("metal-pack"));
FString FMetalCompilerToolchain::MetalStripBinary(TEXT("metal-strip"));
#endif

FString FMetalCompilerToolchain::MetalMapExtension(TEXT(".metalmap"));

FString FMetalCompilerToolchain::XcrunPath(TEXT("/usr/bin/xcrun"));
FString FMetalCompilerToolchain::MetalMacSDK(TEXT("macosx"));
FString FMetalCompilerToolchain::MetalMobileSDK(TEXT("iphoneos"));

FString FMetalCompilerToolchain::WindowsToolchainVersion(TEXT("5.3"));
FString FMetalCompilerToolchain::WindowsToolchainSubversion(TEXT("32023"));

// Static methods

void FMetalCompilerToolchain::CreateAndInit()
{
	Singleton = new FMetalCompilerToolchain;
	Singleton->Init();
}

void FMetalCompilerToolchain::Destroy()
{
	if (Singleton != nullptr)
	{
		Singleton->Teardown();
		delete Singleton;
		Singleton = nullptr;
	}
}

EShaderPlatform FMetalCompilerToolchain::MetalShaderFormatToLegacyShaderPlatform(FName ShaderFormat)
{
	if (ShaderFormat == NAME_SF_METAL_ES3_1_IOS)	return SP_METAL_ES3_1_IOS;
	if (ShaderFormat == NAME_SF_METAL_SM5_IOS)		return SP_METAL_SM5_IOS;
	if (ShaderFormat == NAME_SF_METAL_SM6_IOS)		return SP_METAL_SM6_IOS;
	if (ShaderFormat == NAME_SF_METAL_ES3_1_TVOS)	return SP_METAL_ES3_1_TVOS;
	if (ShaderFormat == NAME_SF_METAL_SM5_TVOS)		return SP_METAL_SM5_TVOS;
	if (ShaderFormat == NAME_SF_METAL_SM5)			return SP_METAL_SM5;
    if (ShaderFormat == NAME_SF_METAL_SM6)          return SP_METAL_SM6;
    if (ShaderFormat == NAME_SF_METAL_SIM)          return SP_METAL_SIM;
	if (ShaderFormat == NAME_SF_METAL_ES3_1)		return SP_METAL_ES3_1;

	return SP_NumPlatforms;
}

// Instance methods
FMetalCompilerToolchain::PackedVersion FMetalCompilerToolchain::GetCompilerVersion(EShaderPlatform Platform) const
{
	if (this->IsMobile(Platform))
	{
		return this->MetalCompilerVersion[AppleSDKMobile];
	}

	return this->MetalCompilerVersion[AppleSDKMac];
}

FMetalCompilerToolchain::PackedVersion FMetalCompilerToolchain::GetTargetVersion(EShaderPlatform Platform) const
{
	if (this->IsMobile(Platform))
	{
		return this->MetalTargetVersion[AppleSDKMobile];
	}

	return this->MetalTargetVersion[AppleSDKMac];
}

const FString& FMetalCompilerToolchain::GetCompilerVersionString(EShaderPlatform Platform) const
{
	if (this->IsMobile(Platform))
	{
		return this->MetalCompilerVersionString[AppleSDKMobile];
	}
	return this->MetalCompilerVersionString[AppleSDKMac];
}

void FMetalCompilerToolchain::Init()
{
	bToolchainAvailable = false;
	bToolchainBinariesPresent = false;
	bSkipPCH = true;

#if PLATFORM_MAC
	EMetalToolchainStatus Result = DoMacNativeSetup();
#else
	EMetalToolchainStatus Result = DoWindowsSetup();
#endif

	if (Result != EMetalToolchainStatus::Success)
	{
		if (GCheckCompilerToolChainSetup > 0)
		{
			UE_LOGF(LogMetalCompilerSetup, Warning, "Metal compiler not found. Shaders will be stored as text.");
		}
		bToolchainAvailable = false;
	}
	else
	{
		Result = FetchCompilerVersion();

		if (Result != EMetalToolchainStatus::Success)
		{
			UE_LOGF(LogMetalCompilerSetup, Log, "Could not parse compiler version.");
		}

		Result = FetchMetalStandardLibraryPath();

		if (Result != EMetalToolchainStatus::Success)
		{
			UE_LOGF(LogMetalCompilerSetup, Warning, "Could not parse metal_stdlib path. Will not use PCH.");
			bSkipPCH = true;
			// This is not really an error since we can compile without the PCH just fine.
			Result = EMetalToolchainStatus::Success;
		}
		else
		{
			// This is forced off for now. If we wish to re-enable it a lot of testing should be done.
			//bSkipPCH = false;
		}

		bToolchainAvailable = true;
	}

	if (GCheckCompilerToolChainSetup > 0)
	{
		if (Result == EMetalToolchainStatus::Success)
		{
			check(IsCompilerAvailable());
			UE_LOGF(LogMetalCompilerSetup, Log, "Metal toolchain setup complete.");
			UE_LOGF(LogMetalCompilerSetup, Log, "Using Local Metal compiler");
			if (!MetalFrontendBinaryCommand[AppleSDKMac].IsEmpty())
			{
				UE_LOGF(LogMetalCompilerSetup, Log, "Mac metalfe found at %ls", *MetalFrontendBinaryCommand[AppleSDKMac]);
			}
			if (!MetalFrontendBinaryCommand[AppleSDKMobile].IsEmpty())
			{
				UE_LOGF(LogMetalCompilerSetup, Log, "Mobile metalfe found at %ls", *MetalFrontendBinaryCommand[AppleSDKMobile]);
			}
			UE_LOGF(LogMetalCompilerSetup, Log, "Mac metalfe version %ls", *MetalCompilerVersionString[AppleSDKMac]);
			UE_LOGF(LogMetalCompilerSetup, Log, "Mobile metalfe version %ls", *MetalCompilerVersionString[AppleSDKMobile]);
		}
		else
		{
			UE_LOGF(LogMetalCompilerSetup, Warning, "Failed to set up Metal toolchain. See log above. Shaders will not be compiled offline.");
		}
	}
}

void FMetalCompilerToolchain::Teardown()
{
	// remove temporaries
	if (this->LocalTempFolder.IsEmpty())
	{
		return;
	}

	if (!FPaths::DirectoryExists(this->LocalTempFolder))
	{
		return;
	}
	
	bool bSuccess = IFileManager::Get().DeleteDirectory(*this->LocalTempFolder, false, true);
	if (!bSuccess)
	{
		UE_LOGF(LogMetalCompilerSetup, Warning, "Could not delete temporary %ls", *this->LocalTempFolder);
	}
}

FMetalCompilerToolchain::EMetalToolchainStatus FMetalCompilerToolchain::FetchCompilerVersion()
{
	EMetalToolchainStatus Result = EMetalToolchainStatus::Success;
	{
		int32 ReturnCode = 0;
		FString StdOut;
		// metal -v writes its output to stderr
		// But the underlying (windows) implementation of CreateProc puts everything into one pipe, which is written to StdOut.
		bool bResult = this->ExecMetalFrontend(AppleSDKMac, TEXT("-v --target=air64-apple-darwin18.7.0"), &ReturnCode, &StdOut, &StdOut);
		check(bResult);
		if (ReturnCode > 0)
		{
			return EMetalToolchainStatus::CouldNotParseCompilerVersion;
		}
		
		Result = ParseCompilerVersionAndTarget(StdOut, this->MetalCompilerVersionString[AppleSDKMac], this->MetalCompilerVersion[AppleSDKMac], this->MetalTargetVersion[AppleSDKMac]);

		if (Result != EMetalToolchainStatus::Success)
		{
			return Result;
		}
	}

	{
		int32 ReturnCode = 0;
		FString StdOut;
		// metal -v writes its output to stderr
		bool bResult = this->ExecMetalFrontend(AppleSDKMobile, TEXT("-v --target=air64-apple-darwin18.7.0"), &ReturnCode, &StdOut, &StdOut);
		check(bResult);
		if (ReturnCode > 0)
		{
			return EMetalToolchainStatus::CouldNotParseCompilerVersion;
		}
		
		Result = ParseCompilerVersionAndTarget(StdOut, this->MetalCompilerVersionString[AppleSDKMobile], this->MetalCompilerVersion[AppleSDKMobile], this->MetalTargetVersion[AppleSDKMobile]);
	}

	return Result;
}

FMetalCompilerToolchain::EMetalToolchainStatus FMetalCompilerToolchain::FetchMetalStandardLibraryPath()
{
	// if we've already decided to skip compiling a PCH we don't need this path at all.
	if (this->bSkipPCH)
	{
		return EMetalToolchainStatus::Success;
	}

	EMetalToolchainStatus Result = EMetalToolchainStatus::Success;
	{
		int32 ReturnCode = 0;
		FString StdOut, StdErr;
		bool bResult = this->ExecMetalFrontend(AppleSDKMac, TEXT("--print-search-dirs"), &ReturnCode, &StdOut, &StdErr);
		check(bResult);
		Result = ParseLibraryToolpath(StdOut, this->MetalStandardLibraryPath[AppleSDKMac]);

		if (Result != EMetalToolchainStatus::Success)
		{
			return Result;
		}
	}

	{
		int32 ReturnCode = 0;
		FString StdOut, StdErr;
		bool bResult = this->ExecMetalFrontend(AppleSDKMobile, TEXT("--print-search-dirs"), &ReturnCode, &StdOut, &StdErr);
		check(bResult);
		Result = ParseLibraryToolpath(StdOut, this->MetalStandardLibraryPath[AppleSDKMobile]);
	}

	return Result;
}

#if PLATFORM_MAC
FMetalCompilerToolchain::EMetalToolchainStatus FMetalCompilerToolchain::DoMacNativeSetup()
{
	FString ToolchainBase;
	if (FParse::Value(FCommandLine::Get(), TEXT("-MetalToolchainOverride="), ToolchainBase))
	{
		const bool bUseOverride = (!ToolchainBase.IsEmpty() && FPaths::DirectoryExists(ToolchainBase));
		if (bUseOverride)
		{
			MetalFrontendBinaryCommand[AppleSDKMac] = ToolchainBase / TEXT("macos") / TEXT("bin") / MetalFrontendBinary;
			MetalFrontendBinaryCommand[AppleSDKMobile] = ToolchainBase / TEXT("ios") / TEXT("bin") / MetalFrontendBinary;

			const bool bIsFrontendPresent = FPaths::FileExists(MetalFrontendBinaryCommand[AppleSDKMac]) && FPaths::FileExists(MetalFrontendBinaryCommand[AppleSDKMobile]);
			if (!bIsFrontendPresent)
			{
				UE_LOGF(LogMetalCompilerSetup, Warning, "Missing Metal frontend in %ls.", *ToolchainBase);
				return EMetalToolchainStatus::ToolchainNotFound;
			}

			MetalArBinaryCommand[AppleSDKMac] = ToolchainBase / TEXT("macos") / TEXT("bin") / MetalArBinary;
			MetalArBinaryCommand[AppleSDKMobile] = ToolchainBase / TEXT("ios") / TEXT("bin") / MetalArBinary;

			MetalLibBinaryCommand[AppleSDKMac] = ToolchainBase / TEXT("macos") / TEXT("bin") / MetalLibraryBinary;
			MetalLibBinaryCommand[AppleSDKMobile] = ToolchainBase / TEXT("ios") / TEXT("bin") / MetalLibraryBinary;

			MetalPackBinaryCommand[AppleSDKMac] = ToolchainBase / TEXT("macos") / TEXT("bin") / MetalPackBinary;
            MetalPackBinaryCommand[AppleSDKMobile] = ToolchainBase / TEXT("ios") / TEXT("bin") / MetalPackBinary;
			
			MetalStripBinaryCommand[AppleSDKMac] = ToolchainBase / TEXT("macos") / TEXT("bin") / MetalStripBinary;
			MetalStripBinaryCommand[AppleSDKMobile] = ToolchainBase / TEXT("ios") / TEXT("bin") / MetalStripBinary;

			if (!FPaths::FileExists(MetalArBinaryCommand[AppleSDKMac]) ||
				!FPaths::FileExists(MetalArBinaryCommand[AppleSDKMobile]) ||
				!FPaths::FileExists(MetalLibBinaryCommand[AppleSDKMac]) ||
				!FPaths::FileExists(MetalLibBinaryCommand[AppleSDKMobile]) ||
				!FPaths::FileExists(MetalLibBinaryCommand[AppleSDKMobile]) ||
                !FPaths::FileExists(MetalPackBinaryCommand[AppleSDKMac]) ||
                !FPaths::FileExists(MetalPackBinaryCommand[AppleSDKMobile]) ||
				!FPaths::FileExists(MetalStripBinaryCommand[AppleSDKMac]) ||
				!FPaths::FileExists(MetalStripBinaryCommand[AppleSDKMobile]))
			{
				UE_LOGF(LogMetalCompilerSetup, Warning, "Missing toolchain binaries in %ls.", *ToolchainBase);
				return EMetalToolchainStatus::ToolchainNotFound;
			}

			this->bToolchainBinariesPresent = true;

			return EMetalToolchainStatus::Success;
		}
	}

	int32 ReturnCode = 0;
	FString StdOut, StdErr;
	bool bSuccess = this->ExecGenericCommand(*XcrunPath, *FString::Printf(TEXT("--sdk %s --find %s"), *this->MetalMacSDK, *this->MetalFrontendBinary), &ReturnCode, &StdOut, &StdErr);
	bSuccess |= FPaths::FileExists(StdOut);
	if(!bSuccess || ReturnCode > 0)
	{
		UE_LOGF(LogMetalCompilerSetup, Warning, "Missing Mac Metal toolchain (macos SDK not found).");
		return EMetalToolchainStatus::ToolchainNotFound;
	}
	
	bSuccess = this->ExecGenericCommand(*XcrunPath, *FString::Printf(TEXT("--sdk %s --find %s"), *this->MetalMobileSDK, *this->MetalFrontendBinary), &ReturnCode, &StdOut, &StdErr);
	bSuccess |= FPaths::FileExists(StdOut);
	if(!bSuccess || ReturnCode > 0)
	{
		UE_LOGF(LogMetalCompilerSetup, Warning, "Missing Mobile Metal toolchain (iphoneos SDK not found).");
		return EMetalToolchainStatus::ToolchainNotFound;
	}

	this->bToolchainBinariesPresent = true;

	return EMetalToolchainStatus::Success;
}
#endif

#if PLATFORM_WINDOWS
FMetalCompilerToolchain::EMetalToolchainStatus FMetalCompilerToolchain::DoWindowsSetup()
{
	FString TargetPlatform;
	if (FParse::Value(FCommandLine::Get(), TEXT("TargetPlatform="), TargetPlatform))
	{
		if (!TargetPlatform.StartsWith(TEXT("IOS"), ESearchCase::IgnoreCase) && 
		    !TargetPlatform.StartsWith(TEXT("Mac"), ESearchCase::IgnoreCase) &&
		    !TargetPlatform.StartsWith(TEXT("tvOS"), ESearchCase::IgnoreCase))
		{
			FPlatformMisc::SetEnvironmentVar(TEXT("NO_METAL"),TEXT("PLEASE"));
			return EMetalToolchainStatus::ToolchainNotFound;
		}
	}
	else if (FPlatformMisc::GetEnvironmentVariable(TEXT("NO_METAL")) == TEXT("PLEASE"))
	{
		return EMetalToolchainStatus::ToolchainNotFound;
	}

	int32 Result = 0;
	
	FString ToolchainBase;

	static const FString SDKRootEnvFar(TEXT("UE_SDKS_ROOT"));
	FString SDKPath = FPlatformMisc::GetEnvironmentVariable(*SDKRootEnvFar);
	FString ProgramFilesPath = FPlatformMisc::GetEnvironmentVariable(TEXT("ProgramFiles"));;

	if (SDKPath.Len() != 0)
	{
		FString HostPlatform(TEXT("HostWin64"));
		ToolchainBase = FPaths::Combine(*SDKPath, *HostPlatform, TEXT("Win64"), TEXT("MetalDeveloperTools"), WindowsToolchainVersion, TEXT("metal"), WindowsToolchainSubversion);
	}
	
	const bool bUseAutoSDK = (!ToolchainBase.IsEmpty() && FPaths::DirectoryExists(ToolchainBase));
	if(!bUseAutoSDK)
	{
		// We expect WindowsMetalToolchainOverride to contains the path up to the folder that contains bin & lib
		GConfig->GetString(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("WindowsMetalToolchainOverride"), ToolchainBase, GEngineIni);

		const bool bUseOverride = (!ToolchainBase.IsEmpty() && FPaths::DirectoryExists(ToolchainBase));
		if (!bUseOverride)
		{
			ToolchainBase = FPaths::Combine(*ProgramFilesPath, TEXT("Metal Developer Tools"), TEXT("metal"), WindowsToolchainSubversion);
		}
	}

	FString MacBinaryBasePath;
	FString IOSBinaryBasePath;

	// Starting Metal Developers Tools 5.0 we have a WindowsToolchainSubversion and mac & ios use the same binary on Windows
	if (WindowsToolchainSubversion.IsEmpty())
	{
		MacBinaryBasePath = ToolchainBase / TEXT("macos") / TEXT("bin");
		IOSBinaryBasePath = ToolchainBase / TEXT("ios") / TEXT("bin");
	}
	else
	{
		MacBinaryBasePath = ToolchainBase / TEXT("bin");
		IOSBinaryBasePath = MacBinaryBasePath;
	}

	MetalFrontendBinaryCommand[AppleSDKMac] = MacBinaryBasePath / MetalFrontendBinary;
	MetalFrontendBinaryCommand[AppleSDKMobile] = IOSBinaryBasePath / MetalFrontendBinary;

	bool bUseLocalMetalToolchain = FPaths::FileExists(MetalFrontendBinaryCommand[AppleSDKMac]) && FPaths::FileExists(MetalFrontendBinaryCommand[AppleSDKMobile]);
	if (!bUseLocalMetalToolchain)
	{
		if (GCheckCompilerToolChainSetup > 0)
		{
			UE_LOGF(LogMetalCompilerSetup, Display, "Searching for Metal toolchain, but it doesn't appear to be installed.");
			UE_LOGF(LogMetalCompilerSetup, Display, "Searched for %ls and %ls", *MetalFrontendBinaryCommand[AppleSDKMac], *MetalFrontendBinaryCommand[AppleSDKMobile]);
		}
		return EMetalToolchainStatus::ToolchainNotFound;
	}

	MetalArBinaryCommand[AppleSDKMac] = MacBinaryBasePath / MetalArBinary;
	MetalArBinaryCommand[AppleSDKMobile] = IOSBinaryBasePath / MetalArBinary;

	MetalLibBinaryCommand[AppleSDKMac] = MacBinaryBasePath / MetalLibraryBinary;
	MetalLibBinaryCommand[AppleSDKMobile] = IOSBinaryBasePath / MetalLibraryBinary;

	MetalPackBinaryCommand[AppleSDKMac] = MacBinaryBasePath / MetalPackBinary;
    MetalPackBinaryCommand[AppleSDKMobile] = IOSBinaryBasePath / MetalPackBinary;
	
	MetalStripBinaryCommand[AppleSDKMac] = MacBinaryBasePath / MetalStripBinary;
	MetalStripBinaryCommand[AppleSDKMobile] = IOSBinaryBasePath / MetalStripBinary;
    
	if (!FPaths::FileExists(MetalArBinaryCommand[AppleSDKMac]) ||
		!FPaths::FileExists(MetalArBinaryCommand[AppleSDKMobile]) ||
		!FPaths::FileExists(MetalLibBinaryCommand[AppleSDKMac]) ||
		!FPaths::FileExists(MetalLibBinaryCommand[AppleSDKMobile]) ||
        !FPaths::FileExists(MetalPackBinaryCommand[AppleSDKMac]) ||
        !FPaths::FileExists(MetalPackBinaryCommand[AppleSDKMobile]) ||
		!FPaths::FileExists(MetalStripBinaryCommand[AppleSDKMac]) ||
		!FPaths::FileExists(MetalStripBinaryCommand[AppleSDKMobile]))
	{
		if (GCheckCompilerToolChainSetup > 0)
		{
			UE_LOGF(LogMetalCompilerSetup, Warning, "Missing toolchain binaries.")
		}
		return EMetalToolchainStatus::ToolchainNotFound;
	}

	this->bToolchainBinariesPresent = true;
	

	return EMetalToolchainStatus::Success;
}
#endif

bool FMetalCompilerToolchain::ExecMetalFrontend(EAppleSDKType SDK, const TCHAR* Parameters, int32* OutReturnCode, FString* OutStdOut, FString* OutStdErr) const
{
	check(this->bToolchainBinariesPresent);
#if PLATFORM_MAC
	if (this->MetalFrontendBinaryCommand[SDK].IsEmpty())
	{
		FString BuiltParams = FString::Printf(TEXT("--sdk %s %s %s"), *SDKToString(SDK), *this->MetalFrontendBinary, Parameters);
		return ExecGenericCommand(*XcrunPath, *BuiltParams, OutReturnCode, OutStdOut, OutStdErr);
	}
	else
#endif
	return ExecGenericCommand(*this->MetalFrontendBinaryCommand[SDK], Parameters, OutReturnCode, OutStdOut, OutStdErr);
}

bool FMetalCompilerToolchain::ExecMetalLib(EAppleSDKType SDK, const TCHAR* Parameters, int32* OutReturnCode, FString* OutStdOut, FString* OutStdErr) const
{
	check(this->bToolchainBinariesPresent);
#if PLATFORM_MAC
	if (this->MetalLibBinaryCommand[SDK].IsEmpty())
	{
		FString BuiltParams = FString::Printf(TEXT("--sdk %s %s %s"), *SDKToString(SDK), *this->MetalLibraryBinary, Parameters);
		return ExecGenericCommand(*XcrunPath, *BuiltParams, OutReturnCode, OutStdOut, OutStdErr);
	}
	else
#endif
	return ExecGenericCommand(*this->MetalLibBinaryCommand[SDK], Parameters, OutReturnCode, OutStdOut, OutStdErr);
}

bool FMetalCompilerToolchain::ExecMetalPack(EAppleSDKType SDK, const TCHAR* Parameters, int32* OutReturnCode, FString* OutStdOut, FString* OutStdErr) const
{
    check(this->bToolchainBinariesPresent);
#if PLATFORM_MAC
    if (this->MetalPackBinaryCommand[SDK].IsEmpty())
    {
        FString BuiltParams = FString::Printf(TEXT("--sdk %s %s %s"), *SDKToString(SDK), *this->MetalPackBinary, Parameters);
        return ExecGenericCommand(*XcrunPath, *BuiltParams, OutReturnCode, OutStdOut, OutStdErr);
    }
    else
#endif
    return ExecGenericCommand(*this->MetalPackBinaryCommand[SDK], Parameters, OutReturnCode, OutStdOut, OutStdErr);
}

bool FMetalCompilerToolchain::ExecMetalAr(EAppleSDKType SDK, const TCHAR* ScriptFile, int32* OutReturnCode, FString* OutStdOut, FString* OutStdErr) const
{
	check(this->bToolchainBinariesPresent);
	// WARNING: This phase may be run in parallel so we must not collide our scripts

	// metal-ar is really llvm-ar, which acts like the standard ar. Since we usually end up with a ton of objects we are archiving we'd like to script it
	// Unfortunately ar reads its script from stdin (when the -M arg is present) instead of being provided a file
	// So on windows we'll spawn cmd.exe and pipe the script file into metal-ar.exe -M
#if PLATFORM_MAC
	FString Command;
	if (this->MetalArBinaryCommand[SDK].IsEmpty())
	{
		Command = FString::Printf(TEXT("-c \"%s -sdk %s '%s' -M < '%s'\""), *XcrunPath, *SDKToString(SDK), *this->MetalArBinary, ScriptFile);
	}
	else
	{
		Command = FString::Printf(TEXT("-c \"'%s' -M < '%s'\""), *this->MetalArBinaryCommand[SDK], ScriptFile);
	}
	bool bSuccess = ExecGenericCommand(TEXT("/bin/sh"), *Command, OutReturnCode, OutStdOut, OutStdErr);
#else
	FString Command = FString::Printf(TEXT("/C type \"%s\" | \"%s\" -M"), ScriptFile, *this->MetalArBinaryCommand[SDK]);
	bool bSuccess = ExecGenericCommand(TEXT("cmd.exe"), *Command, OutReturnCode, OutStdOut, OutStdErr, true /*bIsConsoleApp*/);
#endif
	if (!bSuccess)
	{
		UE_LOGF(LogMetalShaderCompiler, Error, "Error creating .metalar. %ls.", **OutStdOut);
		UE_LOGF(LogMetalShaderCompiler, Error, "Error creating .metalar. %ls.", **OutStdErr);
	}
	return bSuccess;
}

bool FMetalCompilerToolchain::ExecMetalStrip(EAppleSDKType SDK, const TCHAR* Parameters, int32* OutReturnCode, FString* OutStdOut, FString* OutStdErr) const
{
	check(this->bToolchainBinariesPresent);
#if PLATFORM_MAC
	if (this->MetalStripBinaryCommand[SDK].IsEmpty())
	{
		FString BuiltParams = FString::Printf(TEXT("--sdk %s %s %s"), *SDKToString(SDK), *this->MetalStripBinary, Parameters);
		return ExecGenericCommand(*XcrunPath, *BuiltParams, OutReturnCode, OutStdOut, OutStdErr);
	}
	else
#endif
	return ExecGenericCommand(*this->MetalStripBinaryCommand[SDK], Parameters, OutReturnCode, OutStdOut, OutStdErr);
}

bool FMetalCompilerToolchain::ExecGenericCommand(const TCHAR* Command, const TCHAR* Params, int32* OutReturnCode, FString* OutStdOut, FString* OutStdErr, bool bIsConsoleApp) const
{
#if PLATFORM_WINDOWS
	if(bIsConsoleApp)
	{
		// Why do we have our own implementation here? Because metal.exe wants to create a console window. 
		// So if we don't specify the options to CreateProc we end up with tons and tons of windows appearing and disappearing during a cook.
		void* OutputReadPipe = nullptr;
		void* OutputWritePipe = nullptr;
		FPlatformProcess::CreatePipe(OutputReadPipe, OutputWritePipe);
		FProcHandle Proc = FPlatformProcess::CreateProc(Command, Params, false, true, true, nullptr, -1, nullptr, OutputWritePipe, nullptr);

		if (!Proc.IsValid())
		{
			FPlatformProcess::ClosePipe(OutputReadPipe, OutputWritePipe);
			return false;
		}

		int32 RC;
		FPlatformProcess::WaitForProc(Proc);
		FPlatformProcess::GetProcReturnCode(Proc, &RC);
		if (OutStdOut)
		{
			*OutStdOut = FPlatformProcess::ReadPipe(OutputReadPipe);
		}
		FPlatformProcess::ClosePipe(OutputReadPipe, OutputWritePipe);
		FPlatformProcess::CloseProc(Proc);
		if (OutReturnCode)
		{
			*OutReturnCode = RC;
		}

		return RC == 0;
	}
	else
#endif
	{
		// Otherwise use the API
		return FPlatformProcess::ExecProcess(Command, Params, OutReturnCode, OutStdOut, OutStdErr);
	}
}

bool FMetalCompilerToolchain::CompileMetalShader(FMetalShaderBytecodeJob& Job, FMetalShaderBytecode& Output) const
{
	// The local files 
	const FString& LocalInputMetalFilePath = Job.InputFile;
	const FString& LocalOutputMetalAIRFilePath = Job.OutputObjectFile;
	const FString& LocalOutputMetalLibFilePath = Job.OutputFile;

	EAppleSDKType SDK = FMetalCompilerToolchain::MetalFormatToSDK(Job.ShaderFormat);
	
	// .metal -> .air
	FString IncludeArgs = Job.IncludeDir.Len() ? FString::Printf(TEXT("-I %s"), *Job.IncludeDir) : TEXT("");
	{
		// Invoke the metal frontend.
		FString MetalParams = FString::Printf(TEXT("%s %s %s %s -Wno-null-character -fbracket-depth=1024 %s %s %s %s %s -fmodules-cache-path=%s -o %s"), *Job.MinOSVersion, *Job.PreserveInvariance, *Job.DebugInfo, *Job.MathMode, TEXT("-c"), *Job.Standard, *Job.Defines, *IncludeArgs, *LocalInputMetalFilePath, *Job.ModuleCacheDirectory, *LocalOutputMetalAIRFilePath);
		
		if (Job.bOptimizeForSize)
		{
			MetalParams += TEXT(" -Os");
		}
		
		// We don't use incremental build and this void creating temporary files in C drive in Windows
		MetalParams += TEXT(" -fno-temp-file");

		bool bSuccess = this->ExecMetalFrontend(SDK, *MetalParams, &Job.ReturnCode, &Job.Results, &Job.Errors);

		// Delete the cache directory because it takes a approx 5mb per compile
		IFileManager::Get().DeleteDirectory(*Job.ModuleCacheDirectory, false, true);
		
		if (!bSuccess || (Job.ReturnCode != 0))
		{
			Job.Message = FString::Printf(TEXT("Failed to compile %s to bytecode %s, code: %d, output: %s %s"), *LocalInputMetalFilePath, *LocalOutputMetalAIRFilePath, Job.ReturnCode, *Job.Results, *Job.Errors);
			return false;
		}
	}

	{
		// If we have succeeded, now we can create a metallib out of the AIR.
		// TODO do we want to do this in every case? Should be able to skip if we are using Shader Libraries at the high level

		FString MetalLibParams = FString::Printf(TEXT("-o %s %s"), *LocalOutputMetalLibFilePath, *LocalOutputMetalAIRFilePath);
		bool bSuccess = this->ExecMetalLib(SDK, *MetalLibParams, &Job.ReturnCode, &Job.Results, &Job.Errors);
		if (!bSuccess || (Job.ReturnCode != 0))
		{
			Job.Message = FString::Printf(TEXT("Failed to package %s into %s, code: %d, output: %s %s"), *LocalOutputMetalAIRFilePath, *LocalOutputMetalLibFilePath, Job.ReturnCode, *Job.Results, *Job.Errors);
			return false;
		}
	}

	// At this point we have an .air file and a .metallib file
	if (Job.bUseNativeShaderLibrary)
	{
		// Retain the .air.
		bool bSuccess = FFileHelper::LoadFileToArray(Output.ObjectFile, *LocalOutputMetalAIRFilePath);
		
		// Clean up the .air file
		IFileManager::Get().Delete(*LocalOutputMetalAIRFilePath);
		
		if (!bSuccess)
		{
			Job.Message = FString::Printf(TEXT("Failed to store AIR %s"), *LocalOutputMetalAIRFilePath);
			return false;
		}
	}
	else
	{
		// Clean up the .air file
		IFileManager::Get().Delete(*LocalOutputMetalAIRFilePath);
		
		// For non-native shader library we need to strip the metallib now
		if (Job.bStripIndividualMetalLibs)
		{
			if (!StripMetalLib(SDK, LocalOutputMetalLibFilePath, LocalOutputMetalLibFilePath, true))
			{
				Job.Message = FString::Printf(TEXT("Failed to strip metallib %s"), *LocalOutputMetalLibFilePath);
				return false;
			}
		}
	}

	{
		// Retain the .metallib
		Output.NativePath = LocalInputMetalFilePath;
		bool bSuccess = FFileHelper::LoadFileToArray(Output.OutputFile, *LocalOutputMetalLibFilePath);

		// Clean up the .metallib file
		IFileManager::Get().Delete(*LocalOutputMetalLibFilePath);
		
		if (!bSuccess)
		{
			Job.Message = FString::Printf(TEXT("Failed to store metallib %s"), *LocalOutputMetalLibFilePath);
			return false;
		}
	}

	return true;
}

bool FMetalCompilerToolchain::StripMetalLib(EAppleSDKType SDK, const FString& InMetalLibFilePath, const FString& OutMetalLibFilePath, const bool bStripTypes) const
{
	UE_LOGF(LogMetalShaderCompiler, Display, "Stripping lib %ls Size: %lld", *InMetalLibFilePath, IFileManager::Get().FileSize(*InMetalLibFilePath));
	
	FString StripTypesArg = bStripTypes ? FString(TEXT("-T")) : FString(); 
	
	// Strip
	FString MetalStripParams = FString::Printf(TEXT("-S %s -o=\"%s\" \"%s\""), *StripTypesArg, *OutMetalLibFilePath, *InMetalLibFilePath);
	
	int32 ReturnCode = 0;
	FString Results;
	FString Errors;
	bool bSuccess = this->ExecMetalStrip(SDK, *MetalStripParams, &ReturnCode, &Results, &Errors);
	
	if (!bSuccess || ReturnCode != 0)
	{
		UE_LOGF(LogShaders, Error, "Stripping failed: metallib failed with code %d: %ls %ls", ReturnCode, *Results, *Errors);
		return false;
	}
	
	int64 FileSize = IFileManager::Get().FileSize(*OutMetalLibFilePath);
	if (FileSize <= 0)
	{
		UE_LOGF(LogShaders, Error, "Stripping failed: metallib size == %lld: %ls", FileSize, *OutMetalLibFilePath);
		return false;
	}
	UE_LOGF(LogMetalShaderCompiler, Display, "Stripped size of %ls Size: %lld", *OutMetalLibFilePath, FileSize);
	return true;
}
