// Copyright Epic Games, Inc. All Rights Reserved.
// ....

#pragma once

#include "CoreMinimal.h"
#include "RHIDefinitions.h"
#include "HAL/PlatformProcess.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "RHIFeatureLevel.h"
#include "RHIShaderPlatform.h"

// IOS and TVOS use the mobile toolchain.
enum EAppleSDKType
{
	AppleSDKMac,
	AppleSDKMobile,
	AppleSDKCount,
};

static FName NAME_SF_METAL_ES3_1_IOS(TEXT("SF_METAL_ES3_1_IOS"));
static FName NAME_SF_METAL_SM5_IOS(TEXT("SF_METAL_SM5_IOS"));
static FName NAME_SF_METAL_SM6_IOS(TEXT("SF_METAL_SM6_IOS"));
static FName NAME_SF_METAL_ES3_1_TVOS(TEXT("SF_METAL_ES3_1_TVOS"));
static FName NAME_SF_METAL_SM5_TVOS(TEXT("SF_METAL_SM5_TVOS"));
static FName NAME_SF_METAL_SM5(TEXT("SF_METAL_SM5"));
static FName NAME_SF_METAL_SM6(TEXT("SF_METAL_SM6"));
static FName NAME_SF_METAL_SIM(TEXT("SF_METAL_SIM"));
static FName NAME_SF_METAL_ES3_1(TEXT("SF_METAL_ES3_1"));

DECLARE_LOG_CATEGORY_EXTERN(LogMetalCompilerSetup, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogMetalShaderCompiler, Log, All);

class FMetalCompilerToolchain
{
public:
	enum class EMetalToolchainStatus : int32
	{
		Success,
		ToolchainNotFound,
		CouldNotParseCompilerVersion,
		CouldNotParseTargetVersion,
		CouldNotFindMetalStdLib,
	};


	struct PackedVersion
	{
		union
		{
			struct
			{
				int32 Major : 16;
				int32 Minor : 8;
				int32 Patch : 8;
			};
			int32 Version;
		};
	};

	// Initializes this toolchain.
	void Init();
	// Tears down the toolchain
	void Teardown();

	// Takes a Job and compiles a shader. Produces .air and .metallib
	bool CompileMetalShader(struct FMetalShaderBytecodeJob& Job, struct FMetalShaderBytecode& Output) const;
	
	// Strips the Metal lib of debug symbols and optionally type info
	bool StripMetalLib(EAppleSDKType SDK, const FString& InMetalLibFilePath, const FString& OutMetalLibFilePath, const bool bStripTypes) const;
	
	// Executes 'Command' on the local machine
	bool ExecGenericCommand(const TCHAR* Command, const TCHAR* Params, int32* OutReturnCode, FString* OutStdOut, FString* OutStdErr, bool bIsConsoleApp = false) const;
	// Executes the metal frontend compiler for 'SDK' on the local or remote machine, depending on the current configuration
	bool ExecMetalFrontend(EAppleSDKType SDK, const TCHAR* Parameters, int32* OutReturnCode, FString* OutStdOut, FString* OutStdErr) const;
	// Executes metallib for 'SDK' on the local or remote machine, depending on configuration
	bool ExecMetalLib(EAppleSDKType SDK, const TCHAR* Parameters, int32* OutReturnCode, FString* OutStdOut, FString* OutStdErr) const;
	// Executes metal-ar for 'SDK' on the local or remote machine, depending on configuration
	bool ExecMetalAr(EAppleSDKType SDK, const TCHAR* ScriptFile, int32* OutReturnCode, FString* OutStdOut, FString* OutStdErr) const;
	// Executes metal-pack for 'SDK' on the local or remote machine, depending on configuration
    bool ExecMetalPack(EAppleSDKType SDK, const TCHAR* Parameters, int32* OutReturnCode, FString* OutStdOut, FString* OutStdErr) const;
	bool ExecMetalStrip(EAppleSDKType SDK, const TCHAR* Parameters, int32* OutReturnCode, FString* OutStdOut, FString* OutStdErr) const;

	// This toolchain is set up correctly and ready to use.
	bool IsCompilerAvailable() const
	{
		return this->bToolchainAvailable;
	}

	// The version of the compiler, given by metal -v
	PackedVersion GetCompilerVersion(EShaderPlatform Platform) const;

	// The AIR target version, given by metal -v
	PackedVersion GetTargetVersion(EShaderPlatform Platform) const;

	// The first line of metal -v, which gives the version
	const FString& GetCompilerVersionString(EShaderPlatform Platform) const;

	// Fully qualified path to a process-specific temporary directory on the local machine.
	// This directory will be destroyed when the toolchain is destroyed
	const FString& GetLocalTempDir() const
	{
		// nothing like dispatch_once on windows?
		// this should always happen on the same thread anyway
		static bool bLocalTempDirCreated = false;
		if (!bLocalTempDirCreated)
		{
			LocalTempFolder = FPaths::Combine(FPlatformProcess::UserTempDir(), TEXT("MetalShaderCompilation"), FString::Printf(TEXT("%u"), FPlatformProcess::GetCurrentProcessId()));
			if (!FPaths::DirectoryExists(LocalTempFolder))
			{
				bool bSuccess = IFileManager::Get().MakeDirectory(*LocalTempFolder, true);
				if (!bSuccess)
				{
					UE_LOGF(LogMetalCompilerSetup, Fatal, "Attempting to create temporary directory at %ls but failed.", *LocalTempFolder);
				}
			}
			bLocalTempDirCreated = true;
		}

		return LocalTempFolder;
	}

	// The Singleton Toolchain object
	static const FMetalCompilerToolchain* Get()
	{
		return Singleton;
	}

	// Creates and intializes the toolchain for this process
	static void CreateAndInit();

	// Cleans up and deletes the toolchain for this process
	static void Destroy();

	// True if Platform indicates a mobile (ios, tvos) platform
	static bool IsMobile(const EShaderPlatform Platform)
	{
		return (Platform == SP_METAL_ES3_1_IOS ||
				Platform == SP_METAL_SM5_IOS ||
				Platform == SP_METAL_SM6_IOS ||
				Platform == SP_METAL_SIM ||
				Platform == SP_METAL_ES3_1_TVOS ||
				Platform == SP_METAL_SM5_TVOS);
	}

	// True if Format is the FName of a mobile shader format (ios, tvos)
	static bool IsMobile(const FName& Format)
	{
		EShaderPlatform Platform = MetalShaderFormatToLegacyShaderPlatform(Format);
		return IsMobile(Platform);
	}

	// True if Platform is a tvos platform
	static bool IsTVOS(const EShaderPlatform Platform)
	{
		return Platform == SP_METAL_ES3_1_TVOS || Platform == SP_METAL_SM5_TVOS;
	}

	// Converts an FName ShaderFormat to the equivalent ShaderPlatform
	static EShaderPlatform MetalShaderFormatToLegacyShaderPlatform(FName ShaderFormat);
	// Returns the SDK (Mac, Mobile) to use in order to compile shaders of Platform
	static EAppleSDKType MetalShaderPlatformToSDK(EShaderPlatform Platform)
	{
		if (IsMobile(Platform))
		{
			return AppleSDKMobile;
		}
		return AppleSDKMac;
	}

	// Returns the SDK (Mac, Mobile) to use in order to compile shaders of ShaderFormat 
	static EAppleSDKType MetalFormatToSDK(FName ShaderFormat)
	{
		if (IsMobile(ShaderFormat))
		{
			return AppleSDKMobile;
		}
		return AppleSDKMac;
	}
	
	static const FString& SDKToString(EAppleSDKType SDK)
	{
		if(SDK == AppleSDKMac)
		{
			return MetalMacSDK;
		}
		return MetalMobileSDK;
	}

	// The extension of a metal shader - .metal
	static FString MetalExtention;
	// The extension of a packed metal library - .metallib
	static FString MetalLibraryExtension;
	// The extension of the metal IR objects - .air
	static FString MetalObjectExtension;
	// The name of the metal frontend compiler - metal
	static FString MetalFrontendBinary;
	// The name of the metal-ar archiver - metal-ar
	static FString MetalArBinary;
	// The name of the metal binary packager - metallib
	static FString MetalLibraryBinary;
	// The name of metal-pack
    static FString MetalPackBinary;
	// The name of metal-strip
	static FString MetalStripBinary;

	// The extension of the mapping from shader to metallib for shared material libraries - .metalmap
	static FString MetalMapExtension;

	// The path to xcrun
	static FString XcrunPath;
	// The string xcrun expects for the mac SDK - macos
	static FString MetalMacSDK;
	// The string xcrun expects for the mobile SDKs - iphoneos
	static FString MetalMobileSDK;
	// The version of the windows native metal compiler
	static FString WindowsToolchainVersion;
	// The subversion needed for the windows native metal compiler after version 5.0
	static FString WindowsToolchainSubversion;

private:
	// Members
	bool bToolchainAvailable : 1;
	bool bToolchainBinariesPresent	: 1;
	// In this implementation we will also skip PCH generation but this is left in case we'd like to turn it on in the future.
	// Probably needs serious testing as to whether it actually works and actually gains us anything.
	bool bSkipPCH					: 1;
	
	// The path to metal_stdlib
	FString	MetalStandardLibraryPath[AppleSDKCount];

	// These are the strings to pass to Exec to invoke various utilities.
	// On Mac we'll just use xcrun and the name of the utility

	// On Windows, or when using an override on Mac, we will need to figure out the full path to each	
	// The command string to invoke 'metal'
	FString	MetalFrontendBinaryCommand[AppleSDKCount];
	// The command string to invoke 'metallib'
	FString MetalLibBinaryCommand[AppleSDKCount];
	// The command string to invoke 'metal-ar'
	FString MetalArBinaryCommand[AppleSDKCount];

	// The command string to invoke 'metal-pack'
    FString MetalPackBinaryCommand[AppleSDKCount];
	// The command string to invoke 'metal-strip'
	FString MetalStripBinaryCommand[AppleSDKCount];

	// The compiler version string, parsed out of metal -v
	FString	MetalCompilerVersionString[AppleSDKCount];
	// The compiler version number, parsed out of metal -v. This is the first number that occurs - not the number that is (metalfe-###)
	PackedVersion MetalCompilerVersion[AppleSDKCount];
	// The compiler target version, parsed out of metal -v. This is the number from 'Target: air64-apple-darwin##.##.##'
	PackedVersion MetalTargetVersion[AppleSDKCount];

	// Names of temporary directories, generated when the toolchain is initialized

	// mutable :(
	mutable FString LocalTempFolder;

	// Statics
	// The one and only toolchain
	static FMetalCompilerToolchain* Singleton;

#if PLATFORM_MAC
	// fills out the paths and sets up the toolchain for compilation on the local mac
	EMetalToolchainStatus DoMacNativeSetup();
#endif
#if PLATFORM_WINDOWS
	// Sets up toolchain for compilation on windows.
	EMetalToolchainStatus DoWindowsSetup();
#endif

	// Parses and verifies the version of the compiler. Fetched via metal -v
	EMetalToolchainStatus FetchCompilerVersion();
	// Parses and verifies where metal_stdlib is located.
	EMetalToolchainStatus FetchMetalStandardLibraryPath();
};

/*

*/
